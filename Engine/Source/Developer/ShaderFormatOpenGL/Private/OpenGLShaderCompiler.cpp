// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Containers/AnsiString.h"
#include "CrossCompiler.h"
#include "HAL/FileManager.h"
#include "HlslccHeaderWriter.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"
#include "ShaderCompilerCommon.h"
#include "ShaderCompilerDefinitions.h"
#include "ShaderFormatOpenGL.h"
#include "ShaderParameterParser.h"
#include "ShaderPreprocessTypes.h"
#include "SpirvReflectCommon.h"
#include <algorithm>
#include <regex>

#if PLATFORM_WINDOWS
#include "HAL/Platform.h"
#include "Windows/AllowWindowsPlatformTypes.h"
	#include <objbase.h>
	#include <stdio.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif
#include "ShaderCore.h"
#include "ShaderPreprocessor.h"
#include "ShaderCompilerCommon.h"
#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
	#include <GL/glcorearb.h>
	#include <GL/wglext.h>
#include "Windows/HideWindowsPlatformTypes.h"
#elif PLATFORM_LINUX
	#include <GL/glcorearb.h>
	#include <GL/glext.h>
	#include <SDL3/SDL.h>
	#include <stdio.h>
	#include <wchar.h>
	typedef SDL_Window*		SDL_HWindow;
	typedef SDL_GLContext	SDL_HGLContext;
	struct FPlatformOpenGLContext
	{
		SDL_HWindow		hWnd;
		SDL_HGLContext	hGLContext;		//	this is a (void*) pointer
	};
#elif PLATFORM_MAC
	#include <OpenGL/OpenGL.h>
	#include <OpenGL/gl3.h>
	#include <OpenGL/gl3ext.h>
	#ifndef GL_COMPUTE_SHADER
	#define GL_COMPUTE_SHADER 0x91B9
	#endif
	#ifndef GL_TESS_EVALUATION_SHADER
	#define GL_TESS_EVALUATION_SHADER 0x8E87
	#endif
	#ifndef GL_TESS_CONTROL_SHADER
	#define GL_TESS_CONTROL_SHADER 0x8E88
	#endif
#endif

#include "OpenGLShaderResources.h"

THIRD_PARTY_INCLUDES_START
#include "spirv_reflect.h"
#include <map>
THIRD_PARTY_INCLUDES_END

DEFINE_LOG_CATEGORY_STATIC(LogOpenGLShaderCompiler, Log, All);

#define VALIDATE_GLSL_WITH_DRIVER		0
#define ENABLE_IMAGINATION_COMPILER		1

/*------------------------------------------------------------------------------
	Shader compiling.
------------------------------------------------------------------------------*/

#if PLATFORM_WINDOWS
/** List all OpenGL entry points needed for shader compilation. */
#define ENUM_GL_ENTRYPOINTS(EnumMacro) \
	EnumMacro(PFNGLCOMPILESHADERPROC,glCompileShader) \
	EnumMacro(PFNGLCREATESHADERPROC,glCreateShader) \
	EnumMacro(PFNGLDELETESHADERPROC,glDeleteShader) \
	EnumMacro(PFNGLGETSHADERIVPROC,glGetShaderiv) \
	EnumMacro(PFNGLGETSHADERINFOLOGPROC,glGetShaderInfoLog) \
	EnumMacro(PFNGLSHADERSOURCEPROC,glShaderSource) \
	EnumMacro(PFNGLDELETEBUFFERSPROC,glDeleteBuffers)

/** Define all GL functions. */
#define DEFINE_GL_ENTRYPOINTS(Type,Func) static Type Func = NULL;
ENUM_GL_ENTRYPOINTS(DEFINE_GL_ENTRYPOINTS);

/** This function is handled separately because it is used to get a real context. */
static PFNWGLCREATECONTEXTATTRIBSARBPROC wglCreateContextAttribsARB;

/** Platform specific OpenGL context. */
struct FPlatformOpenGLContext
{
	HWND WindowHandle;
	HDC DeviceContext;
	HGLRC OpenGLContext;
};

/**
 * A dummy wndproc.
 */
static LRESULT CALLBACK PlatformDummyGLWndproc(HWND hWnd, uint32 Message, WPARAM wParam, LPARAM lParam)
{
	return DefWindowProc(hWnd, Message, wParam, lParam);
}

/**
 * Initialize a pixel format descriptor for the given window handle.
 */
static void PlatformInitPixelFormatForDevice(HDC DeviceContext)
{
	// Pixel format descriptor for the context.
	PIXELFORMATDESCRIPTOR PixelFormatDesc;
	FMemory::Memzero(PixelFormatDesc);
	PixelFormatDesc.nSize		= sizeof(PIXELFORMATDESCRIPTOR);
	PixelFormatDesc.nVersion	= 1;
	PixelFormatDesc.dwFlags		= PFD_DRAW_TO_WINDOW | PFD_SUPPORT_OPENGL | PFD_DOUBLEBUFFER;
	PixelFormatDesc.iPixelType	= PFD_TYPE_RGBA;
	PixelFormatDesc.cColorBits	= 32;
	PixelFormatDesc.cDepthBits	= 0;
	PixelFormatDesc.cStencilBits	= 0;
	PixelFormatDesc.iLayerType	= PFD_MAIN_PLANE;

	// Set the pixel format and create the context.
	int32 PixelFormat = ChoosePixelFormat(DeviceContext, &PixelFormatDesc);
	if (!PixelFormat || !SetPixelFormat(DeviceContext, PixelFormat, &PixelFormatDesc))
	{
		UE_LOGF(LogOpenGLShaderCompiler, Fatal,"Failed to set pixel format for device context.");
	}
}

/**
 * Create a dummy window used to construct OpenGL contexts.
 */
static void PlatformCreateDummyGLWindow(FPlatformOpenGLContext* OutContext)
{
	const TCHAR* WindowClassName = TEXT("DummyGLToolsWindow");

	// Register a dummy window class.
	static bool bInitializedWindowClass = false;
	if (!bInitializedWindowClass)
	{
		WNDCLASS wc;

		bInitializedWindowClass = true;
		FMemory::Memzero(wc);
		wc.style = CS_OWNDC;
		wc.lpfnWndProc = PlatformDummyGLWndproc;
		wc.cbClsExtra = 0;
		wc.cbWndExtra = 0;
		wc.hInstance = NULL;
		wc.hIcon = NULL;
		wc.hCursor = NULL;
		wc.hbrBackground = (HBRUSH)(COLOR_MENUTEXT);
		wc.lpszMenuName = NULL;
		wc.lpszClassName = WindowClassName;
		ATOM ClassAtom = ::RegisterClass(&wc);
		check(ClassAtom);
	}

	// Create a dummy window.
	OutContext->WindowHandle = CreateWindowEx(
		WS_EX_WINDOWEDGE,
		WindowClassName,
		NULL,
		WS_POPUP,
		CW_USEDEFAULT, CW_USEDEFAULT, CW_USEDEFAULT, CW_USEDEFAULT,
		NULL, NULL, NULL, NULL);
	check(OutContext->WindowHandle);

	// Get the device context.
	OutContext->DeviceContext = GetDC(OutContext->WindowHandle);
	check(OutContext->DeviceContext);
	PlatformInitPixelFormatForDevice(OutContext->DeviceContext);
}

/**
 * Create a core profile OpenGL context.
 */
static void PlatformCreateOpenGLContextCore(FPlatformOpenGLContext* OutContext, int MajorVersion, int MinorVersion, HGLRC InParentContext)
{
	check(wglCreateContextAttribsARB);
	check(OutContext);
	check(OutContext->DeviceContext);

	int AttribList[] =
	{
		WGL_CONTEXT_MAJOR_VERSION_ARB, MajorVersion,
		WGL_CONTEXT_MINOR_VERSION_ARB, MinorVersion,
		WGL_CONTEXT_FLAGS_ARB, WGL_CONTEXT_FORWARD_COMPATIBLE_BIT_ARB | WGL_CONTEXT_DEBUG_BIT_ARB,
		WGL_CONTEXT_PROFILE_MASK_ARB, WGL_CONTEXT_CORE_PROFILE_BIT_ARB,
		0
	};

	OutContext->OpenGLContext = wglCreateContextAttribsARB(OutContext->DeviceContext, InParentContext, AttribList);
	check(OutContext->OpenGLContext);
}

/**
 * Make the context current.
 */
static void PlatformMakeGLContextCurrent(FPlatformOpenGLContext* Context)
{
	check(Context && Context->OpenGLContext && Context->DeviceContext);
	wglMakeCurrent(Context->DeviceContext, Context->OpenGLContext);
}

/**
 * Initialize an OpenGL context so that shaders can be compiled.
 */
static void PlatformInitOpenGL(void*& ContextPtr, void*& PrevContextPtr, int InMajorVersion, int InMinorVersion)
{
	static FPlatformOpenGLContext ShaderCompileContext = {0};

	ContextPtr = (void*)wglGetCurrentDC();
	PrevContextPtr = (void*)wglGetCurrentContext();

	if (ShaderCompileContext.OpenGLContext == NULL && InMajorVersion && InMinorVersion)
	{
		PlatformCreateDummyGLWindow(&ShaderCompileContext);

		// Disable warning C4191: 'type cast' : unsafe conversion from 'PROC' to 'XXX' while getting GL entry points.
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

		if (wglCreateContextAttribsARB == NULL)
		{
			// Create a dummy context so that wglCreateContextAttribsARB can be initialized.
			ShaderCompileContext.OpenGLContext = wglCreateContext(ShaderCompileContext.DeviceContext);
			check(ShaderCompileContext.OpenGLContext);
			PlatformMakeGLContextCurrent(&ShaderCompileContext);
			wglCreateContextAttribsARB = (PFNWGLCREATECONTEXTATTRIBSARBPROC)wglGetProcAddress("wglCreateContextAttribsARB");
			check(wglCreateContextAttribsARB);
			wglDeleteContext(ShaderCompileContext.OpenGLContext);
		}

		// Create a context so that remaining GL function pointers can be initialized.
		PlatformCreateOpenGLContextCore(&ShaderCompileContext, InMajorVersion, InMinorVersion, /*InParentContext=*/ NULL);
		check(ShaderCompileContext.OpenGLContext);
		PlatformMakeGLContextCurrent(&ShaderCompileContext);

		if (glCreateShader == NULL)
		{
			// Initialize all entry points.
			#define GET_GL_ENTRYPOINTS(Type,Func) Func = (Type)wglGetProcAddress(#Func);
			ENUM_GL_ENTRYPOINTS(GET_GL_ENTRYPOINTS);

			// Check that all of the entry points have been initialized.
			bool bFoundAllEntryPoints = true;
			#define CHECK_GL_ENTRYPOINTS(Type,Func) if (Func == NULL) { bFoundAllEntryPoints = false; UE_LOGF(LogOpenGLShaderCompiler, Warning, "Failed to find entry point for %ls", TEXT(#Func)); }
			ENUM_GL_ENTRYPOINTS(CHECK_GL_ENTRYPOINTS);
			checkf(bFoundAllEntryPoints, TEXT("Failed to find all OpenGL entry points."));
		}

		// Restore warning C4191.
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
	}
	PlatformMakeGLContextCurrent(&ShaderCompileContext);
}
static void PlatformReleaseOpenGL(void* ContextPtr, void* PrevContextPtr)
{
	wglMakeCurrent((HDC)ContextPtr, (HGLRC)PrevContextPtr);
}
#elif PLATFORM_LINUX
/** List all OpenGL entry points needed for shader compilation. */
#define ENUM_GL_ENTRYPOINTS(EnumMacro) \
	EnumMacro(PFNGLCOMPILESHADERPROC,glCompileShader) \
	EnumMacro(PFNGLCREATESHADERPROC,glCreateShader) \
	EnumMacro(PFNGLDELETESHADERPROC,glDeleteShader) \
	EnumMacro(PFNGLGETSHADERIVPROC,glGetShaderiv) \
	EnumMacro(PFNGLGETSHADERINFOLOGPROC,glGetShaderInfoLog) \
	EnumMacro(PFNGLSHADERSOURCEPROC,glShaderSource) \
	EnumMacro(PFNGLDELETEBUFFERSPROC,glDeleteBuffers)

/** Define all GL functions. */
// We need to make pointer names different from GL functions otherwise we may end up getting
// addresses of those symbols when looking for extensions.
namespace GLFuncPointers
{
	#define DEFINE_GL_ENTRYPOINTS(Type,Func) static Type Func = NULL;
	ENUM_GL_ENTRYPOINTS(DEFINE_GL_ENTRYPOINTS);
};

using namespace GLFuncPointers;

static void _PlatformCreateDummyGLWindow(FPlatformOpenGLContext *OutContext)
{
	static bool bInitializedWindowClass = false;

	// Create a dummy window.
	OutContext->hWnd = SDL_CreateWindow(NULL,
		1, 1,
		SDL_WINDOW_OPENGL | SDL_WINDOW_BORDERLESS | SDL_WINDOW_HIDDEN | SDL_WINDOW_UTILITY );
}

static void _PlatformCreateOpenGLContextCore(FPlatformOpenGLContext* OutContext)
{
	check(OutContext);
	SDL_HWindow PrevWindow = SDL_GL_GetCurrentWindow();
	SDL_HGLContext PrevContext = SDL_GL_GetCurrentContext();

	OutContext->hGLContext = SDL_GL_CreateContext(OutContext->hWnd);
	SDL_GL_MakeCurrent(PrevWindow, PrevContext);
}

static void _ContextMakeCurrent(SDL_HWindow hWnd, SDL_HGLContext hGLDC)
{
	GLint Result = SDL_GL_MakeCurrent( hWnd, hGLDC );
	check(!Result);
}

static void PlatformInitOpenGL(void*& ContextPtr, void*& PrevContextPtr, int InMajorVersion, int InMinorVersion)
{
	static bool bInitialized = (SDL_GL_GetCurrentWindow() != NULL) && (SDL_GL_GetCurrentContext() != NULL);

	if (!bInitialized)
	{
		check(InMajorVersion > 3 || (InMajorVersion == 3 && InMinorVersion >= 2));
		if (SDL_WasInit(0) == 0)
		{
			SDL_Init(SDL_INIT_VIDEO);
		}
		else
		{
			Uint32 InitializedSubsystemsMask = SDL_WasInit(SDL_INIT_VIDEO);
			if ((InitializedSubsystemsMask & SDL_INIT_VIDEO) == 0)
			{
				SDL_InitSubSystem(SDL_INIT_VIDEO);
			}
		}

		if (SDL_GL_LoadLibrary(NULL))
		{
			UE_LOGF(LogOpenGLShaderCompiler, Fatal, "Unable to dynamically load libGL: %ls", ANSI_TO_TCHAR(SDL_GetError()));
		}

		if (glCreateShader == nullptr)
		{
			// Initialize all entry points.
			#define GET_GL_ENTRYPOINTS(Type,Func) GLFuncPointers::Func = reinterpret_cast<Type>(SDL_GL_GetProcAddress(#Func));
			ENUM_GL_ENTRYPOINTS(GET_GL_ENTRYPOINTS);

			// Check that all of the entry points have been initialized.
			bool bFoundAllEntryPoints = true;
			#define CHECK_GL_ENTRYPOINTS(Type,Func) if (Func == nullptr) { bFoundAllEntryPoints = false; UE_LOGF(LogOpenGLShaderCompiler, Warning, "Failed to find entry point for %ls", TEXT(#Func)); }
			ENUM_GL_ENTRYPOINTS(CHECK_GL_ENTRYPOINTS);
			checkf(bFoundAllEntryPoints, TEXT("Failed to find all OpenGL entry points."));
		}

		if	(SDL_GL_SetAttribute( SDL_GL_CONTEXT_MAJOR_VERSION, InMajorVersion))
		{
			UE_LOGF(LogOpenGLShaderCompiler, Fatal, "Failed to set GL major version: %ls", ANSI_TO_TCHAR(SDL_GetError()));
		}

		if	(SDL_GL_SetAttribute( SDL_GL_CONTEXT_MINOR_VERSION, InMinorVersion))
		{
			UE_LOGF(LogOpenGLShaderCompiler, Fatal, "Failed to set GL minor version: %ls", ANSI_TO_TCHAR(SDL_GetError()));
		}

		if	(SDL_GL_SetAttribute( SDL_GL_CONTEXT_FLAGS, SDL_GL_CONTEXT_FORWARD_COMPATIBLE_FLAG))
		{
			UE_LOGF(LogOpenGLShaderCompiler, Fatal, "Failed to set GL flags: %ls", ANSI_TO_TCHAR(SDL_GetError()));
		}

		if	(SDL_GL_SetAttribute( SDL_GL_CONTEXT_PROFILE_MASK, SDL_GL_CONTEXT_PROFILE_CORE))
		{
			UE_LOGF(LogOpenGLShaderCompiler, Fatal, "Failed to set GL mask/profile: %ls", ANSI_TO_TCHAR(SDL_GetError()));
		}

		// Create a dummy context to verify opengl support.
		FPlatformOpenGLContext DummyContext;
		_PlatformCreateDummyGLWindow(&DummyContext);
		_PlatformCreateOpenGLContextCore(&DummyContext);

		if (DummyContext.hGLContext)
		{
			_ContextMakeCurrent(DummyContext.hWnd, DummyContext.hGLContext);
		}
		else
		{
			UE_LOGF(LogOpenGLShaderCompiler, Fatal, "OpenGL %d.%d not supported by driver", InMajorVersion, InMinorVersion);
			return;
		}

		PrevContextPtr = NULL;
		ContextPtr = DummyContext.hGLContext;
		bInitialized = true;
	}

	PrevContextPtr = reinterpret_cast<void*>(SDL_GL_GetCurrentContext());
	SDL_HGLContext NewContext = SDL_GL_CreateContext(SDL_GL_GetCurrentWindow());
	SDL_GL_MakeCurrent(SDL_GL_GetCurrentWindow(), NewContext);
	ContextPtr = reinterpret_cast<void*>(NewContext);
}

static void PlatformReleaseOpenGL(void* ContextPtr, void* PrevContextPtr)
{
	SDL_GL_MakeCurrent(SDL_GL_GetCurrentWindow(), reinterpret_cast<SDL_HGLContext>(PrevContextPtr));
	SDL_GL_DestroyContext(reinterpret_cast<SDL_HGLContext>(ContextPtr));
}
#elif PLATFORM_MAC
static void PlatformInitOpenGL(void*& ContextPtr, void*& PrevContextPtr, int InMajorVersion, int InMinorVersion)
{
	check(InMajorVersion > 3 || (InMajorVersion == 3 && InMinorVersion >= 2));

	CGLPixelFormatAttribute AttribList[] =
	{
		kCGLPFANoRecovery,
		kCGLPFAAccelerated,
		kCGLPFAOpenGLProfile,
		(CGLPixelFormatAttribute)kCGLOGLPVersion_3_2_Core,
		(CGLPixelFormatAttribute)0
	};

	CGLPixelFormatObj PixelFormat;
	GLint NumFormats = 0;
	CGLError Error = CGLChoosePixelFormat(AttribList, &PixelFormat, &NumFormats);
	check(Error == kCGLNoError);

	CGLContextObj ShaderCompileContext;
	Error = CGLCreateContext(PixelFormat, NULL, &ShaderCompileContext);
	check(Error == kCGLNoError);

	Error = CGLDestroyPixelFormat(PixelFormat);
	check(Error == kCGLNoError);

	PrevContextPtr = (void*)CGLGetCurrentContext();

	Error = CGLSetCurrentContext(ShaderCompileContext);
	check(Error == kCGLNoError);

	ContextPtr = (void*)ShaderCompileContext;
}
static void PlatformReleaseOpenGL(void* ContextPtr, void* PrevContextPtr)
{
	CGLContextObj ShaderCompileContext = (CGLContextObj)ContextPtr;
	CGLContextObj PreviousShaderCompileContext = (CGLContextObj)PrevContextPtr;
	CGLError Error;

	Error = CGLSetCurrentContext(PreviousShaderCompileContext);
	check(Error == kCGLNoError);

	Error = CGLDestroyContext(ShaderCompileContext);
	check(Error == kCGLNoError);
}
#endif

/** Map shader frequency -> GL shader type. */
GLenum GLFrequencyTable[] =
{
	GL_VERTEX_SHADER,	// SF_Vertex
	GLenum(0), // SF_Mesh
	GLenum(0), // SF_Amplification
	GL_FRAGMENT_SHADER, // SF_Pixel
	GL_GEOMETRY_SHADER,	// SF_Geometry
	GL_COMPUTE_SHADER,  // SF_Compute
	// Ray tracing shaders are not supported in OpenGL
	GLenum(0), // SF_RayGen
	GLenum(0), // SF_RayMiss
	GLenum(0), // SF_RayHitGroup (closest hit, any hit, intersection)
	GLenum(0), // SF_RayCallable
	// Work graph shaders are not supported in OpenGL
	GLenum(0), // SF_WorkGraphRoot
	GLenum(0), // SF_WorkGraphComputeNode
};

static_assert(UE_ARRAY_COUNT(GLFrequencyTable) == SF_NumFrequencies, "Frequency table size mismatch.");

static inline bool IsDigit(TCHAR Char)
{
	return Char >= '0' && Char <= '9';
}

/**
 * Parse a GLSL error.
 * @param OutErrors - Storage for shader compiler errors.
 * @param InLine - A single line from the compile error log.
 */
void ParseGlslError(TArray<FShaderCompilerError>& OutErrors, const FString& InLine)
{
	const TCHAR* ErrorPrefix = TEXT("error: 0:");
	const TCHAR* p = *InLine;
	if (FCString::Strnicmp(p, ErrorPrefix, 9) == 0)
	{
		FString ErrorMsg;
		int32 LineNumber = 0;
		p += FCString::Strlen(ErrorPrefix);

		// Skip to a number, take that to be the line number.
		while (*p && !IsDigit(*p)) { p++; }
		while (*p && IsDigit(*p))
		{
			LineNumber = 10 * LineNumber + (*p++ - TEXT('0'));
		}

		// Skip to the next alphanumeric value, treat that as the error message.
		while (*p && !FChar::IsAlnum(*p)) { p++; }
		ErrorMsg = p;

		// Generate a compiler error.
		if (ErrorMsg.Len() > 0)
		{
			// Note that no mapping exists from the GLSL source to the original
			// HLSL source.
			FShaderCompilerError& CompilerError = OutErrors.AddDefaulted_GetRef();
			CompilerError.StrippedErrorMessage = FString::Printf(
				TEXT("driver compile error(%d): %s"),
				LineNumber,
				*ErrorMsg
				);
		}
	}
}

static TArray<ANSICHAR> ParseIdentifierANSI(const FString& Str)
{
	TArray<ANSICHAR> Result;
	Result.Reserve(Str.Len());
	for (int32 Index = 0; Index < Str.Len(); ++Index)
	{
		Result.Add(FChar::ToLower((ANSICHAR)Str[Index]));
	}
	Result.Add('\0');

	return Result;
}

static uint32 ParseNumber(const TCHAR* Str)
{
	uint32 Num = 0;
	while (*Str && IsDigit(*Str))
	{
		Num = Num * 10 + *Str++ - '0';
	}
	return Num;
}

enum class EPlatformType
{
	Android,
	IOS,
	Web,
	Desktop
};

struct FDeviceCapabilities
{
	EPlatformType TargetPlatform = EPlatformType::Android;
};

static bool PlatformSupportsOfflineCompilationInternal(const GLSLVersion ShaderVersion);
static void FillDeviceCapsOfflineCompilationInternal(struct FDeviceCapabilities& Capabilities, const GLSLVersion ShaderVersion);
static bool MoveHashLines(FAnsiString& Destination, FAnsiString& Source);
static FAnsiString PrepareCodeForOfflineCompilationInternal(const GLSLVersion ShaderVersion, EShaderFrequency Frequency, const ANSICHAR* InShaderSource);
static void PlatformCompileOfflineInternal(const FShaderCompilerInput& Input, FShaderCompilerOutput& ShaderOutput, const ANSICHAR* ShaderSource, const GLSLVersion ShaderVersion);
static void CompileOfflineInternal(const FShaderCompilerInput& Input, FShaderCompilerOutput& Output, const GLSLVersion ShaderVersion, const ANSICHAR* InShaderSource);

/**
 * Construct the final microcode from the compiled and verified shader source.
 * @param ShaderOutput - Where to store the microcode and parameter map.
 * @param ShaderInput - The input struct for the shader being compiled.
 * @param InShaderSource - GLSL source with input/output signature.
 * @param SourceLen - The length of the GLSL source code.
 * @param Version - The GLSL version to target.
 */
void BuildShaderOutputInternal(
	FShaderCompilerOutput& ShaderOutput,
	const FShaderCompilerInput& ShaderInput,
	const FShaderParameterParser& ShaderParameterParser,
	const ANSICHAR* InShaderSource,
	int32 SourceLen,
	GLSLVersion Version)
{
	const ANSICHAR* USFSource = InShaderSource;
	CrossCompiler::FHlslccHeader CCHeader;
	if (!CCHeader.Read(USFSource, SourceLen))
	{
		UE_LOGF(LogOpenGLShaderCompiler, Error, "Bad hlslcc header found");
	}

	if (*USFSource != '#')
	{
		UE_LOGF(LogOpenGLShaderCompiler, Error, "Bad hlslcc header found! Missing '#'!");
	}

	FOpenGLCodeHeader Header = {0};
	FShaderResourceTable SRT {};
	EShaderFrequency Frequency = (EShaderFrequency)ShaderOutput.Target.Frequency;

	TBitArray<> UsedUniformBufferSlots;
	UsedUniformBufferSlots.Init(false, 32);

	// Write out the magic markers.
	Header.GlslMarker = 0x474c534c;
	switch (Frequency)
	{
	case SF_Vertex:
		Header.FrequencyMarker = 0x5653;
		break;
	case SF_Pixel:
		Header.FrequencyMarker = 0x5053;
		break;
	case SF_Geometry:
		Header.FrequencyMarker = 0x4753;
		break;
	case SF_Compute:
		Header.FrequencyMarker = 0x4353;
		break;
	default:
		UE_LOGF(LogOpenGLShaderCompiler, Fatal, "Invalid shader frequency: %d", (int32)Frequency);
	}

	static const FString AttributePrefix = TEXT("in_ATTRIBUTE");
	static const FString AttributeVarPrefix = TEXT("in_var_ATTRIBUTE");
	static const FString GL_Prefix = TEXT("gl_");
	for (auto& Input : CCHeader.Inputs)
	{
		// Only process attributes for vertex shaders.
		if (Frequency == SF_Vertex && Input.Name.StartsWith(AttributePrefix))
		{
			int32 AttributeIndex = ParseNumber(*Input.Name + AttributePrefix.Len());
			Header.Bindings.InOutMask.EnableField(AttributeIndex);
		}
		else if (Frequency == SF_Vertex && Input.Name.StartsWith(AttributeVarPrefix))
		{
			int32 AttributeIndex = ParseNumber(*Input.Name + AttributeVarPrefix.Len());
			Header.Bindings.InOutMask.EnableField(AttributeIndex);
		}
		// Record user-defined input varyings
		else if (!Input.Name.StartsWith(GL_Prefix))
		{
			FOpenGLShaderVarying Var;
			Var.Location = Input.Index;
			Var.Varying = ParseIdentifierANSI(Input.Name);
			Header.Bindings.InputVaryings.Add(Var);
		}
	}

	static const FString TargetPrefix = "out_Target";
	static const FString GL_FragDepth = "gl_FragDepth";
	for (auto& Output : CCHeader.Outputs)
	{
		// Only targets for pixel shaders must be tracked.
		if (Frequency == SF_Pixel && Output.Name.StartsWith(TargetPrefix))
		{
			uint8 TargetIndex = ParseNumber(*Output.Name + TargetPrefix.Len());
			Header.Bindings.InOutMask.EnableField(TargetIndex);
		}
		// Only depth writes for pixel shaders must be tracked.
		else if (Frequency == SF_Pixel && Output.Name.Equals(GL_FragDepth))
		{
			Header.Bindings.InOutMask.EnableField(CrossCompiler::FShaderBindingInOutMask::DepthStencilMaskIndex);
		}
		// Record user-defined output varyings
		else if (!Output.Name.StartsWith(GL_Prefix))
		{
			FOpenGLShaderVarying Var;
			Var.Location = Output.Index;
			Var.Varying = ParseIdentifierANSI(Output.Name);
			Header.Bindings.OutputVaryings.Add(Var);
		}
	}

	TMap<FString, FString> BindingNameMap;

	// Then 'normal' uniform buffers.
	for (auto& UniformBlock : CCHeader.UniformBlocks)
	{
		uint16 UBIndex = UniformBlock.Index;

		UsedUniformBufferSlots[UBIndex] = true;
		HandleReflectedUniformBuffer(ShaderInput, UniformBlock.Name, UBIndex, ShaderOutput);
		Header.Bindings.NumUniformBuffers++;
	}

	const uint16 BytesPerComponent = 4;


	FString GlobalIgnoreStrings[] =
	{
		TEXT("gl_LastFragDepthARM"),
		TEXT("ARM_shader_framebuffer_fetch_depth_stencil"),
	};

	// Packed global uniforms
	TMap<ANSICHAR, uint16> PackedGlobalArraySize;
	for (auto& PackedGlobal : CCHeader.PackedGlobals)
	{
		bool bIgnore = false;
		for (uint32_t i = 0; i < UE_ARRAY_COUNT(GlobalIgnoreStrings); ++i)
		{
			if (PackedGlobal.Name.StartsWith(GlobalIgnoreStrings[i]))
			{
				bIgnore = true;
				break;
			}
		}

		if (bIgnore)
			continue;

		HandleReflectedGlobalConstantBufferMember(
			ShaderInput,
			ShaderParameterParser,
			PackedGlobal.Name,
			PackedGlobal.PackedType,
			PackedGlobal.Offset* BytesPerComponent,
			PackedGlobal.Count* BytesPerComponent,
			ShaderOutput
		);

		uint16& Size = PackedGlobalArraySize.FindOrAdd(PackedGlobal.PackedType);
		Size = FMath::Max<uint16>(BytesPerComponent * (PackedGlobal.Offset + PackedGlobal.Count), Size);
	}

	// Packed Uniform Buffers
	TMap<int, TMap<ANSICHAR, uint16> > PackedUniformBuffersSize;
	for (auto& PackedUB : CCHeader.PackedUBs)
	{
		UsedUniformBufferSlots[PackedUB.Attribute.Index] = true;
		HandleReflectedUniformBuffer(ShaderInput, PackedUB.Attribute.Name, PackedUB.Attribute.Index, ShaderOutput);
		Header.Bindings.NumUniformBuffers++;

		// Nothing else...
		//for (auto& Member : PackedUB.Members)
		//{
		//}
	}

	// Packed Uniform Buffers copy lists & setup sizes for each UB/Precision entry
	enum EFlattenUBState
	{
		Unknown,
		GroupedUBs,
		FlattenedUBs,
	};
	EFlattenUBState UBState = Unknown;
	for (auto& PackedUBCopy : CCHeader.PackedUBCopies)
	{
		CrossCompiler::FUniformBufferCopyInfo CopyInfo;
		CopyInfo.SourceUBIndex = PackedUBCopy.SourceUB;
		CopyInfo.SourceOffsetInFloats = PackedUBCopy.SourceOffset;
		CopyInfo.DestUBIndex = PackedUBCopy.DestUB;
		CopyInfo.DestUBTypeName = PackedUBCopy.DestPackedType;
		CopyInfo.DestUBTypeIndex = CrossCompiler::PackedTypeNameToTypeIndex(CopyInfo.DestUBTypeName);
		CopyInfo.DestOffsetInFloats = PackedUBCopy.DestOffset;
		CopyInfo.SizeInFloats = PackedUBCopy.Count;

		Header.UniformBuffersCopyInfo.Add(CopyInfo);

		auto& UniformBufferSize = PackedUniformBuffersSize.FindOrAdd(CopyInfo.DestUBIndex);
		uint16& Size = UniformBufferSize.FindOrAdd(CopyInfo.DestUBTypeName);
		Size = FMath::Max<uint16>(BytesPerComponent * (CopyInfo.DestOffsetInFloats + CopyInfo.SizeInFloats), Size);

		check(UBState == Unknown || UBState == GroupedUBs);
		UBState = GroupedUBs;
	}

	for (auto& PackedUBCopy : CCHeader.PackedUBGlobalCopies)
	{
		CrossCompiler::FUniformBufferCopyInfo CopyInfo;
		CopyInfo.SourceUBIndex = PackedUBCopy.SourceUB;
		CopyInfo.SourceOffsetInFloats = PackedUBCopy.SourceOffset;
		CopyInfo.DestUBIndex = PackedUBCopy.DestUB;
		CopyInfo.DestUBTypeName = PackedUBCopy.DestPackedType;
		CopyInfo.DestUBTypeIndex = CrossCompiler::PackedTypeNameToTypeIndex(CopyInfo.DestUBTypeName);
		CopyInfo.DestOffsetInFloats = PackedUBCopy.DestOffset;
		CopyInfo.SizeInFloats = PackedUBCopy.Count;

		Header.UniformBuffersCopyInfo.Add(CopyInfo);

		uint16& Size = PackedGlobalArraySize.FindOrAdd(CopyInfo.DestUBTypeName);
		Size = FMath::Max<uint16>(BytesPerComponent * (CopyInfo.DestOffsetInFloats + CopyInfo.SizeInFloats), Size);

		check(UBState == Unknown || UBState == FlattenedUBs);
		UBState = FlattenedUBs;
	}

	Header.Bindings.bFlattenUB = (UBState == FlattenedUBs);

	// Setup Packed Array info
	Header.Bindings.PackedGlobalArrays.Reserve(PackedGlobalArraySize.Num());
	for (auto Iterator = PackedGlobalArraySize.CreateIterator(); Iterator; ++Iterator)
	{
		ANSICHAR TypeName = Iterator.Key();
		uint16 Size = Iterator.Value();
		Size = (Size + 0xf) & (~0xf);
		CrossCompiler::FPackedArrayInfo Info;
		Info.Size = Size;
		Info.TypeName = TypeName;
		Info.TypeIndex = CrossCompiler::PackedTypeNameToTypeIndex(TypeName);
		Header.Bindings.PackedGlobalArrays.Add(Info);
	}

	// Setup Packed Uniform Buffers info
	Header.Bindings.PackedUniformBuffers.Reserve(PackedUniformBuffersSize.Num());
	for (auto Iterator = PackedUniformBuffersSize.CreateIterator(); Iterator; ++Iterator)
	{
		auto& ArraySizes = Iterator.Value();
		TArray<CrossCompiler::FPackedArrayInfo> InfoArray;
		InfoArray.Reserve(ArraySizes.Num());
		for (auto IterSizes = ArraySizes.CreateIterator(); IterSizes; ++IterSizes)
		{
			ANSICHAR TypeName = IterSizes.Key();
			uint16 Size = IterSizes.Value();
			Size = (Size + 0xf) & (~0xf);
			CrossCompiler::FPackedArrayInfo Info;
			Info.Size = Size;
			Info.TypeName = TypeName;
			Info.TypeIndex = CrossCompiler::PackedTypeNameToTypeIndex(TypeName);
			InfoArray.Add(Info);
		}
		
		// Sort by TypeIndex as expected by eUB uloading code
		InfoArray.Sort([](const CrossCompiler::FPackedArrayInfo& A, const CrossCompiler::FPackedArrayInfo& B)
		{ 
			return A.TypeIndex < B.TypeIndex; 
		});
		
		Header.Bindings.PackedUniformBuffers.Add(InfoArray);
	}

	// Then samplers.
	for (auto& Sampler : CCHeader.Samplers)
	{
		HandleReflectedShaderResource(Sampler.Name, Sampler.Offset, Sampler.Count, ShaderOutput);

		Header.Bindings.NumSamplers = FMath::Max<uint8>(
			Header.Bindings.NumSamplers,
			Sampler.Offset + Sampler.Count
			);

		for (auto& SamplerState : Sampler.SamplerStates)
		{
			HandleReflectedShaderSampler(SamplerState, Sampler.Offset, Sampler.Count, ShaderOutput);
		}
	}

	// Then UAVs (images in GLSL)
	for (auto& UAV : CCHeader.UAVs)
	{
		HandleReflectedShaderUAV(UAV.Name, UAV.Offset, UAV.Count, ShaderOutput);

		Header.Bindings.NumUAVs = FMath::Max<uint8>(
			Header.Bindings.NumSamplers,
			UAV.Offset + UAV.Count
			);
	}

	Header.ShaderName = CCHeader.Name;
	ShaderOutput.bSucceeded = true;

	// Build the SRT for this shader.
	{
		// Build the generic SRT for this shader.
		FShaderCompilerResourceTable GenericSRT;
		BuildResourceTableMapping(ShaderInput.Environment.ResourceTableMap, ShaderInput.Environment.UniformBufferMap, UsedUniformBufferSlots, ShaderOutput.ParameterMap, GenericSRT);
		CullGlobalUniformBuffers(ShaderInput.Environment.UniformBufferMap, ShaderOutput.ParameterMap);

		UE::ShaderCompilerCommon::BuildShaderResourceTable(GenericSRT, SRT);
	}

	constexpr int32 MaxSamplers = 16;

	if (Header.Bindings.NumSamplers > MaxSamplers)
	{
		ShaderOutput.bSucceeded = false;
		FShaderCompilerError& NewError = ShaderOutput.Errors.AddDefaulted_GetRef();
		NewError.StrippedErrorMessage =
			FString::Printf(TEXT("shader uses %d samplers exceeding the limit of %d"),
				Header.Bindings.NumSamplers, MaxSamplers);
	}
	else if (ShaderOutput.bSucceeded)
	{
		// Write out the header
		FMemoryWriter Ar(ShaderOutput.ShaderCode.GetWriteAccess(), true);
		Header.Serialize(Ar, SRT);

		Ar.Serialize((void*)USFSource, SourceLen + 1 - (USFSource - InShaderSource));

		ShaderOutput.ModifiedShaderSource = USFSource;

		if (ShaderInput.Environment.CompilerFlags.Contains(CFLAG_ExtraShaderData))
		{
			ShaderOutput.ShaderCode.AddOptionalData(FShaderCodeName::Key, TCHAR_TO_UTF8(*ShaderInput.GenerateShaderName()));
		}

		// if available, attempt run an offline compilation and extract statistics
		if (ShaderInput.ExtraSettings.OfflineCompilerPath.Len() > 0)
		{
			CompileOfflineInternal(ShaderInput, ShaderOutput, Version, USFSource);
		}
		else
		{
			ShaderOutput.NumInstructions = 0;
		}

		ShaderOutput.NumTextureSamplers = Header.Bindings.NumSamplers;
	}
}

static void ConvertOpenGLVersionFromGLSLVersionInternal(GLSLVersion InVersion, int& OutMajorVersion, int& OutMinorVersion)
{
	switch(InVersion)
	{
		case GLSL_150_ES3_1:
			OutMajorVersion = 3;
			OutMinorVersion = 2;
			break;
		case GLSL_ES3_1_ANDROID:
			OutMajorVersion = 0;
			OutMinorVersion = 0;
			break;
		default:
			// Invalid enum
			check(0);
			OutMajorVersion = 0;
			OutMinorVersion = 0;
			break;
	}
}

/**
 * Precompile a GLSL shader.
 * @param ShaderOutput - The precompiled shader.
 * @param ShaderInput - The shader input.
 * @param ShaderSource - The preprocessed source code.
 * @param Version - The GLSL language version to target.
 * @param Frequency - The shader stage
 */
static void PrecompileShaderInternal(FShaderCompilerOutput& ShaderOutput, const FShaderCompilerInput& ShaderInput, const FShaderParameterParser& ShaderParameterParser, const ANSICHAR* ShaderSource, GLSLVersion Version)
{
	check(ShaderInput.Target.Frequency < SF_NumFrequencies);

	// Lookup the GL shader type.
	GLenum GLFrequency = GLFrequencyTable[ShaderInput.Target.Frequency];
	if (GLFrequency == GL_NONE)
	{
		ShaderOutput.bSucceeded = false;
		FShaderCompilerError& NewError = ShaderOutput.Errors.AddDefaulted_GetRef();
		NewError.StrippedErrorMessage = FString::Printf(TEXT("%s shaders not supported for use in OpenGL."), CrossCompiler::GetFrequencyName((EShaderFrequency)ShaderInput.Target.Frequency));
		return;
	}


	// Create the shader with the preprocessed source code.
	void* ContextPtr;
	void* PrevContextPtr;
	int MajorVersion = 0;
	int MinorVersion = 0;
	ConvertOpenGLVersionFromGLSLVersionInternal(Version, MajorVersion, MinorVersion);
	PlatformInitOpenGL(ContextPtr, PrevContextPtr, MajorVersion, MinorVersion);

	GLint SourceLen = FCStringAnsi::Strlen(ShaderSource);
	GLuint Shader = glCreateShader(GLFrequency);
	{
		const GLchar* SourcePtr = ShaderSource;
		glShaderSource(Shader, 1, &SourcePtr, &SourceLen);
	}

	// Compile and get results.
	glCompileShader(Shader);
	{
		GLint CompileStatus;
		glGetShaderiv(Shader, GL_COMPILE_STATUS, &CompileStatus);
		if (CompileStatus == GL_TRUE)
		{
			ShaderOutput.Target = ShaderInput.Target;
			BuildShaderOutputInternal(
				ShaderOutput,
				ShaderInput,
				ShaderParameterParser,
				ShaderSource,
				(int32)SourceLen,
				Version
				);
		}
		else
		{
			GLint LogLength;
			glGetShaderiv(Shader, GL_INFO_LOG_LENGTH, &LogLength);
			if (LogLength > 1)
			{
				TArray<ANSICHAR> RawCompileLog;
				FString CompileLog;
				TArray<FString> LogLines;

				RawCompileLog.Empty(LogLength);
				RawCompileLog.AddZeroed(LogLength);
				glGetShaderInfoLog(Shader, LogLength, /*OutLength=*/ NULL, RawCompileLog.GetData());
				CompileLog = ANSI_TO_TCHAR(RawCompileLog.GetData());
				CompileLog.ParseIntoArray(LogLines, TEXT("\n"), true);

				for (int32 Line = 0; Line < LogLines.Num(); ++Line)
				{
					ParseGlslError(ShaderOutput.Errors, LogLines[Line]);
				}

				if (ShaderOutput.Errors.Num() == 0)
				{
					FShaderCompilerError& NewError = ShaderOutput.Errors.AddDefaulted_GetRef();
					NewError.StrippedErrorMessage = FString::Printf(
						TEXT("GLSL source:\n%sGL compile log: %s\n"),
						ANSI_TO_TCHAR(ShaderSource),
						ANSI_TO_TCHAR(RawCompileLog.GetData())
						);
				}
			}
			else
			{
				FShaderCompilerError& NewError = ShaderOutput.Errors.AddDefaulted_GetRef();
				NewError.StrippedErrorMessage = TEXT("Shader compile failed without errors.");
			}

			ShaderOutput.bSucceeded = false;
		}
	}
	glDeleteShader(Shader);
	PlatformReleaseOpenGL(ContextPtr, PrevContextPtr);
}

static const ANSICHAR* GetFrequencyPrefix(EShaderFrequency Frequency)
{
	switch (Frequency)
	{
	case SF_Vertex:		return "v";
	case SF_Pixel:		return "p";
	case SF_Geometry:	return "g";
	case SF_Compute:	return "c";
	default:			return "";
	}
}

struct PackedUBMemberInfo
{
	std::string Name;
	std::string SanitizedName;
	std::string TypeQualifier;
	uint32_t SrcOffset;
	uint32_t DestOffset;
	uint32_t SrcSizeInFloats;
	uint32_t DestSizeInFloats;
};

void WritePackedUBHeader(CrossCompiler::FHlslccHeaderWriter& CCHeaderWriter, const TMap<uint32_t, TArray<PackedUBMemberInfo>>& UBMemberInfo, const TMap<uint32, std::string>& UBNames)
{
	for (const auto & Pair : UBNames)
	{
		FString Name(Pair.Value.c_str());
		CCHeaderWriter.WritePackedUB(Name, Pair.Key);
	}

	for (const auto & Pair : UBMemberInfo)
	{
		FString UBName(UBNames[Pair.Key].c_str());
		
		std::string CurTypeQualifier = "\0";
		uint32_t CurSrcOffset = 0;
		uint32_t NextSrcOffset = 0;
		uint32_t CurDestOffset = 0;
		uint32_t NextDestOffset = 0;
		uint32_t TotalSize = 0;

		// Write out UB Copy data
		// Groups copies together to save time on upload
		for (int32_t MemberIdx = 0; MemberIdx < UBMemberInfo[Pair.Key].Num(); MemberIdx++)
		{
			const PackedUBMemberInfo& MemberInfo = UBMemberInfo[Pair.Key][MemberIdx];
			CCHeaderWriter.WritePackedUBField(UBName, UTF8_TO_TCHAR(MemberInfo.SanitizedName.c_str()), MemberInfo.SrcOffset, MemberInfo.DestSizeInFloats * sizeof(float));

			if (TotalSize == 0)
			{
				CurTypeQualifier = MemberInfo.TypeQualifier;
				CurSrcOffset = MemberInfo.SrcOffset / sizeof(float);
				CurDestOffset = MemberInfo.DestOffset / sizeof(float);
			}
			else if(CurTypeQualifier[0] != MemberInfo.TypeQualifier[0] ||
					NextSrcOffset != MemberInfo.SrcOffset / sizeof(float) ||
					NextDestOffset != MemberInfo.DestOffset / sizeof(float))
			{
				// Write out data before starting new 
				CCHeaderWriter.WritePackedUBCopy(Pair.Key, CurSrcOffset, Pair.Key, CurTypeQualifier[0], CurDestOffset, TotalSize, true);

				CurTypeQualifier = MemberInfo.TypeQualifier;
				TotalSize = 0;
				CurSrcOffset = MemberInfo.SrcOffset / sizeof(float);
				CurDestOffset = MemberInfo.DestOffset / sizeof(float);
			}

			TotalSize += MemberInfo.DestSizeInFloats;
			NextSrcOffset = MemberInfo.SrcOffset / sizeof(float) + MemberInfo.DestSizeInFloats;
			NextDestOffset = MemberInfo.DestOffset / sizeof(float) + MemberInfo.DestSizeInFloats;
		}

		// Write out any final data
		if (TotalSize > 0)
		{
			CCHeaderWriter.WritePackedUBCopy(Pair.Key, CurSrcOffset, Pair.Key, CurTypeQualifier[0], CurDestOffset, TotalSize, true);
		}
	}
}

void GetSpvVarQualifier(const SpvReflectBlockVariable& Member, FString & Out)
{
	auto const type = *Member.type_description;

	FString TypeQualifier;

	uint32_t masked_type = type.type_flags & 0xF;

	switch (masked_type)
	{
	default: checkf(false, TEXT("unsupported component type %d"), masked_type); break;
	case SPV_REFLECT_TYPE_FLAG_BOOL:
	case SPV_REFLECT_TYPE_FLAG_INT: 
		Out = (type.traits.numeric.scalar.signedness ? TEXT("i") : TEXT("u"));
		break;
	case SPV_REFLECT_TYPE_FLAG_FLOAT: 
		if (Member.decoration_flags & SPV_REFLECT_DECORATION_RELAXED_PRECISION)
		{
			Out = TEXT("m");
		}
		else
		{
			Out = TEXT("h");
		}
		break;
	}
}

// Adds a member to ne included in the PackedUB structures and generates the #define for readability in glsl
void AddMemberToPackedUB(const std::string& FrequencyPrefix,
	const SpvReflectBlockVariable& Member,
	const std::string& UBName,
	int32_t Index,
	TMap<FString, uint32>& Offsets,
	TArray<PackedUBMemberInfo>& MemberInfos,
	TArray<std::string>& Remap,
	TArray<std::string>& Arrays)
{
	std::string ArrayName;
	const uint32 MbrSize = Member.size / sizeof(float);

	FString TypeQualifier;

	GetSpvVarQualifier(Member, TypeQualifier);

	std::string SanitizedName = Member.name;
	std::replace(SanitizedName.begin(), SanitizedName.end(), '.', '_');

	uint32& Offset = Offsets.FindOrAdd(TypeQualifier);

	auto const type = *Member.type_description;

	bool const bArray = type.traits.array.dims_count > 0;
	bool const bGlobals = Index == -1;

	std::string Name = "#define ";

	if (bGlobals)
	{
		Name += "_Globals_";
	}

	std::string OffsetString = std::to_string(Offset);
	Name += SanitizedName;

	PackedUBMemberInfo& MemberInfo = MemberInfos.AddDefaulted_GetRef();
	MemberInfo.Name = Member.name;
	MemberInfo.SanitizedName = SanitizedName;
	MemberInfo.TypeQualifier = TCHAR_TO_UTF8(*TypeQualifier);
	MemberInfo.SrcOffset = Member.offset;
	MemberInfo.DestOffset = Offset * 4 * sizeof(float);
	MemberInfo.SrcSizeInFloats = Member.size / sizeof(float);
	MemberInfo.DestSizeInFloats = Member.size / sizeof(float);

	if (bArray)
	{
		if (bGlobals)
		{
			ArrayName = UBName + SanitizedName;
		}
		else
		{
			ArrayName = SanitizedName;
		}

		Name += "(Offset)";
		if (type.op == SpvOpTypeMatrix || (type.traits.numeric.matrix.column_count == 4 && type.traits.numeric.matrix.row_count == 4))
		{
			OffsetString += " + (int(Offset) * 4)";
		}
		else
		{
			OffsetString += " + int(Offset)";
		}
	}

	Name += " (";

	std::string UniformPrefix = FrequencyPrefix;
	if (!bGlobals)
	{
		UniformPrefix += std::string("c") + std::to_string(Index);
	}
	else
	{
		UniformPrefix += "u";
	}

	if (type.op == SpvOpTypeMatrix || (type.traits.numeric.matrix.column_count == 4 && type.traits.numeric.matrix.row_count == 4))
	{
		if ((type.traits.numeric.matrix.column_count == 4 && type.traits.numeric.matrix.row_count == 4))
		{
			Name += "mat4(";
		}
		else
		{
			std::string Buff = "mat" + std::to_string(type.traits.numeric.matrix.column_count) + "x" + std::to_string(type.traits.numeric.matrix.row_count) + "(";
			Name += Buff;
		}

		for (uint32_t i = 0; i < type.traits.numeric.matrix.column_count; ++i)
		{
			if (i > 0)
			{
				Name += ",";
			}

			Name += UniformPrefix;
			Name += "_";
			Name += TCHAR_TO_UTF8(*TypeQualifier);

			std::string Buff = "[" + OffsetString + " + " + std::to_string(i) + "]";
			Name += Buff;

			switch (type.traits.numeric.matrix.row_count)
			{
			case 0:
			case 1:
				Name += ".x";
				break;
			case 2:
				Name += ".xy";
				break;
			case 3:
				Name += ".xyz";
				break;
			case 4:
			default:
				Name += ".xyzw";
				break;
			}
		}

		Name += ")";
	}
	else
	{
		Name += UniformPrefix;
		Name += "_";
		Name += TCHAR_TO_UTF8(*TypeQualifier);
		Name += "[";
		Name += OffsetString;
		Name += "]";
		switch (type.traits.numeric.vector.component_count)
		{
		case 0:
		case 1:
			Name += ".x";
			break;
		case 2:
			Name += ".xy";
			break;
		case 3:
			Name += ".xyz";
			break;
		case 4:
		default:
			break;
		}
	}
	Name += ")\n";

	if (bArray)
	{
		Arrays.Add(ArrayName);
	}

	Remap.Add(Name);

	Offset += Align(MbrSize, 4) / 4;
}

void GetPackedUniformString(std::string& OutputString, const std::string& UniformPrefix, const FString& Key, uint32_t Index)
{
	if (Key == TEXT("u"))
	{
		OutputString = "uniform uvec4 ";
		OutputString += UniformPrefix;
		OutputString += "_u[";
		OutputString += std::to_string(Index);
		OutputString += "];\n";
	}
	else if (Key == TEXT("i"))
	{
		OutputString = "uniform ivec4 ";
		OutputString += UniformPrefix;
		OutputString += "_i[";
		OutputString += std::to_string(Index);
		OutputString += "];\n";
	}
	else if (Key == TEXT("h"))
	{
		OutputString = "uniform highp vec4 ";
		OutputString += UniformPrefix;
		OutputString += "_h[";
		OutputString += std::to_string(Index);
		OutputString += "];\n";
	}
	else if (Key == TEXT("m"))
	{
		OutputString = "uniform mediump vec4 ";
		OutputString += UniformPrefix;
		OutputString += "_m[";
		OutputString += std::to_string(Index);
		OutputString += "];\n";
	}
}

struct ReflectionData
{
	TArray<FString> Textures;
	TArray<FString> Samplers;
	TArray<std::string> UAVs;
	TArray<std::string> StructuredBuffers;
	
	std::map<std::string, std::string> UniformVarNames;
	std::map<std::string, std::vector<std::string>> UniformVarMemberNames;

	TMap<FString, uint32> GlobalOffsets;
	TArray<std::string> GlobalRemap;
	TArray<std::string> GlobalArrays;
	TArray<PackedUBMemberInfo> GlobalMemberInfos;

	TMap<uint32, TMap<FString, uint32>> PackedUBOffsets;
	TMap<uint32, TArray<std::string>> PackedUBRemap;
	TMap<uint32, TArray<std::string>> PackedUBArrays;
	TMap<uint32, TArray<PackedUBMemberInfo>> PackedUBMemberInfos;
	TMap<uint32, std::string> PackedUBNames;

	TArray<FString> InputVarNames;
	TArray<FString> OutputVarNames;
};

void ParseReflectionData(const FShaderCompilerInput& ShaderInput, CrossCompiler::FHlslccHeaderWriter& CCHeaderWriter, ReflectionData& ReflectionOut, TArray<uint32>& SpirvData, 
							spv_reflect::ShaderModule& Reflection,	const ANSICHAR* SPIRV_DummySamplerName, const EShaderFrequency Frequency, bool bEmulatedUBs)
{
	const ANSICHAR* FrequencyPrefix = GetFrequencyPrefix(Frequency);

	check(Reflection.GetResult() == SPV_REFLECT_RESULT_SUCCESS);

	SpvReflectResult SPVRResult = SPV_REFLECT_RESULT_NOT_READY;
	TArray<SpvReflectBlockVariable*> ConstantBindings;
	const uint32 GlobalSetId = 32;

	FSpirvReflectBindings ReflectionBindings;
	ReflectionBindings.GatherDescriptorBindings(Reflection);

	uint32 BufferIndices = 0xffffffff;
	uint32 UAVIndices = 0xffffffff;
	// The GLSL backend shares texture slots between SRV and UAV, so we need more than 32 slots.
	// Exceeding the SRV limit of 32 samplers should not be handled by this function, so use 64 bit.
	uint64 TextureIndices = 0xffffffffffffffffllu;
	uint32 UBOIndices = 0xffffffff;
	uint32 SamplerIndices = 0xffffffff;

	for (auto const& Binding : ReflectionBindings.TBufferUAVs)
	{
		check(UAVIndices);
		uint32 Index = FPlatformMath::CountTrailingZeros(UAVIndices);

		// UAVs always claim all slots so we don't have conflicts as D3D expects 0-7
		BufferIndices &= ~(1u << Index);
		TextureIndices &= ~(1llu << uint64(Index));
		UAVIndices &= ~(1u << Index);

		CCHeaderWriter.WriteUAV(UTF8_TO_TCHAR(Binding->name), Index);

		SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
		check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);

		ReflectionOut.UAVs.Add(Binding->name);
	}

	for (auto const& Binding : ReflectionBindings.SBufferUAVs)
	{
		check(UAVIndices);
		uint32 Index = FPlatformMath::CountTrailingZeros(UAVIndices);

		// UAVs always claim all slots so we don't have conflicts as D3D expects 0-7
		BufferIndices &= ~(1u << Index);
		TextureIndices &= ~(1llu << uint64(Index));
		UAVIndices &= ~(1u << Index);

		CCHeaderWriter.WriteUAV(UTF8_TO_TCHAR(Binding->name), Index);
		ReflectionOut.StructuredBuffers.Add(Binding->name);

		SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
		check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
	}

	for (auto const& Binding : ReflectionBindings.TextureUAVs)
	{
		check(UAVIndices);
		uint32 Index = FPlatformMath::CountTrailingZeros(UAVIndices);

		// UAVs always claim all slots so we don't have conflicts as D3D expects 0-7
		// For texture2d this allows us to emulate atomics with buffers
		BufferIndices &= ~(1u << Index);
		TextureIndices &= ~(1llu << uint64(Index));
		UAVIndices &= ~(1u << Index);

		CCHeaderWriter.WriteUAV(UTF8_TO_TCHAR(Binding->name), Index);

		SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
		check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);

		ReflectionOut.UAVs.Add(Binding->name);
	}

	for (auto const& Binding : ReflectionBindings.TBufferSRVs)
	{
		check(TextureIndices);
		uint32 Index = static_cast<uint32>(FPlatformMath::CountTrailingZeros64(TextureIndices));

		// No support for 3-component types in dxc/SPIRV/MSL - need to expose my workarounds there too
		TextureIndices &= ~(1llu << uint64(Index));

		ReflectionOut.Textures.Add(UTF8_TO_TCHAR(Binding->name));

		SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
		check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
	}

	for (int32_t SBufferIndex = ReflectionBindings.SBufferSRVs.Num() - 1; SBufferIndex >= 0; SBufferIndex--)
	{
		auto Binding = ReflectionBindings.SBufferSRVs[SBufferIndex];
		check(BufferIndices);
		uint32 Index = FPlatformMath::CountTrailingZeros(BufferIndices);

		BufferIndices &= ~(1u << Index);

		SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
		check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);

		CCHeaderWriter.WriteUAV(UTF8_TO_TCHAR(Binding->name), Index);
		ReflectionOut.StructuredBuffers.Add(Binding->name);
	}

	TArray<const SpvReflectDescriptorBinding*> RealUniformBuffers;
	for (auto const& Binding : ReflectionBindings.UniformBuffers)
	{
		// Global uniform buffer - handled specially as we care about the internal layout
		if (strstr(Binding->name, "$Globals"))
		{
			TMap<FString, uint32_t> GlobalOffsetSizes;
			for (uint32 i = 0; i < Binding->block.member_count; i++)
			{
				SpvReflectBlockVariable& member = Binding->block.members[i];

				FString TypeQualifier;
				GetSpvVarQualifier(member, TypeQualifier);

				uint32_t& GlobalOffsetSize = GlobalOffsetSizes.FindOrAdd(TypeQualifier);

				if (strstr(member.name, "gl_") || !strcmp(member.name, "ARM_shader_framebuffer_fetch") || !strcmp(member.name, "ARM_shader_framebuffer_fetch_depth_stencil"))
				{
					continue;
				}

				bool bHalfPrecision = member.decoration_flags & SPV_REFLECT_DECORATION_RELAXED_PRECISION;
				CCHeaderWriter.WritePackedGlobal(ANSI_TO_TCHAR(member.name), CrossCompiler::FHlslccHeaderWriter::EncodePackedGlobalType(*(member.type_description), bHalfPrecision), GlobalOffsetSize, member.size);
				GlobalOffsetSize += Align(member.size, 16);

				AddMemberToPackedUB(FrequencyPrefix,
					member,
					"_Globals_",
					-1,
					ReflectionOut.GlobalOffsets,
					ReflectionOut.GlobalMemberInfos,
					ReflectionOut.GlobalRemap,
					ReflectionOut.GlobalArrays);
			}
		}
		else
		{
			const FUniformBufferEntry* UniformBufferEntry = ShaderInput.Environment.UniformBufferMap.Find(Binding->name);

			if (bEmulatedUBs && (UniformBufferEntry == nullptr || !EnumHasAnyFlags(UniformBufferEntry->Flags, ERHIUniformBufferFlags::NoEmulatedUniformBuffer)))
			{
				check(UBOIndices);
				uint32 Index = FPlatformMath::CountTrailingZeros(UBOIndices);
				UBOIndices &= ~(1u << Index);

				ReflectionOut.PackedUBOffsets.Add(Index);
				ReflectionOut.PackedUBRemap.Add(Index);
				ReflectionOut.PackedUBArrays.Add(Index);
				ReflectionOut.PackedUBMemberInfos.Add(Index);
				ReflectionOut.PackedUBNames.Add(Index, std::string(Binding->name));

				for (uint32 i = 0; i < Binding->block.member_count; i++)
				{
					SpvReflectBlockVariable& member = Binding->block.members[i];

					std::string UBName = "_" + std::string(Binding->name) + "_";

					if (member.type_description->type_flags & SPV_REFLECT_TYPE_FLAG_STRUCT)
					{
						for (uint32 n = 0; n < member.member_count; n++)
						{
							// Clone struct member and rename it for struct
							SpvReflectBlockVariable StructMember = member.members[n];

							std::string StructMemberName = std::string(member.name) + "." + std::string(StructMember.name);
							StructMember.name = StructMemberName.c_str();

							AddMemberToPackedUB(FrequencyPrefix,
								StructMember,
								UBName,
								Index,
								ReflectionOut.PackedUBOffsets[Index],
								ReflectionOut.PackedUBMemberInfos[Index],
								ReflectionOut.PackedUBRemap[Index],
								ReflectionOut.PackedUBArrays[Index]);
						}
					}
					else
					{
						AddMemberToPackedUB(FrequencyPrefix,
							member,
							UBName,
							Index,
							ReflectionOut.PackedUBOffsets[Index],
							ReflectionOut.PackedUBMemberInfos[Index],
							ReflectionOut.PackedUBRemap[Index],
							ReflectionOut.PackedUBArrays[Index]);
					}
				}

				SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
				check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
			}
			else
			{
				RealUniformBuffers.Add(Binding);
			}
		}
	}

	//Always write the real uniform buffers after the emulated because OpenGLShaders expects emulated to be in consecutive slots
	for (auto const& Binding : RealUniformBuffers)
	{
		uint32 Index = FPlatformMath::CountTrailingZeros(UBOIndices);
		UBOIndices &= ~(1u << Index);

		std::string OldName = Binding->name;
		std::string NewName = FrequencyPrefix;
		NewName += "b";
		NewName += std::to_string(Index);
		ReflectionOut.UniformVarNames[OldName] = NewName;
		// Regular uniform buffer - we only care about the binding index
		CCHeaderWriter.WriteUniformBlock(UTF8_TO_TCHAR(Binding->name), Index);

		ReflectionOut.UniformVarMemberNames.insert({ OldName, {} });
		for (uint32 i = 0; i < Binding->block.member_count; i++)
		{
			SpvReflectBlockVariable& member = Binding->block.members[i];
			ReflectionOut.UniformVarMemberNames[OldName].push_back(member.name);
		}

		SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
		check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
	}

	WritePackedUBHeader(CCHeaderWriter, ReflectionOut.PackedUBMemberInfos, ReflectionOut.PackedUBNames);

	for (auto const& Binding : ReflectionBindings.TextureSRVs)
	{
		check(TextureIndices);
		uint32 Index = static_cast<uint32>(FPlatformMath::CountTrailingZeros64(TextureIndices));
		TextureIndices &= ~(1llu << uint64(Index));

		ReflectionOut.Textures.Add(UTF8_TO_TCHAR(Binding->name));

		SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
		check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
	}

	for (auto const& Binding : ReflectionBindings.Samplers)
	{
		check(SamplerIndices);
		uint32 Index = FPlatformMath::CountTrailingZeros(SamplerIndices);
		SamplerIndices &= ~(1u << Index);

		ReflectionOut.Samplers.Add(UTF8_TO_TCHAR(Binding->name));

		SPVRResult = Reflection.ChangeDescriptorBindingNumbers(Binding, Index, GlobalSetId);
		check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
	}

	// Spirv-cross can add a dummy sampler for glsl which isn't in the reflection bindings
	ReflectionOut.Samplers.Add(SPIRV_DummySamplerName);

	{
		uint32 Count = 0;
		SPVRResult = Reflection.EnumeratePushConstantBlocks(&Count, nullptr);
		check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
		ConstantBindings.SetNum(Count);
		SPVRResult = Reflection.EnumeratePushConstantBlocks(&Count, ConstantBindings.GetData());
		check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
		if (Count > 0)
		{
			for (auto const& Var : ConstantBindings)
			{
				// Global uniform buffer - handled specially as we care about the internal layout
				if (strstr(Var->name, "$Globals"))
				{
					TMap<FString, uint32_t> GlobalOffsetSizes;

					for (uint32 i = 0; i < Var->member_count; i++)
					{
						SpvReflectBlockVariable& member = Var->members[i];

						FString TypeQualifier;
						GetSpvVarQualifier(member, TypeQualifier);

						uint32_t& GlobalOffsetSize = GlobalOffsetSizes.FindOrAdd(TypeQualifier);

						if (!strcmp(member.name, "gl_FragColor") || !strcmp(member.name, "gl_LastFragColorARM") || !strcmp(member.name, "gl_LastFragDepthARM") || !strcmp(member.name, "ARM_shader_framebuffer_fetch") || !strcmp(member.name, "ARM_shader_framebuffer_fetch_depth_stencil"))
						{
							continue;
						}

						bool bHalfPrecision = member.decoration_flags & SPV_REFLECT_DECORATION_RELAXED_PRECISION;
						CCHeaderWriter.WritePackedGlobal(ANSI_TO_TCHAR(member.name), CrossCompiler::FHlslccHeaderWriter::EncodePackedGlobalType(*(member.type_description), bHalfPrecision), GlobalOffsetSize, member.size);
						GlobalOffsetSize += Align(member.size, 16);

						AddMemberToPackedUB(FrequencyPrefix,
							member,
							"_Globals_",
							-1,
							ReflectionOut.GlobalOffsets,
							ReflectionOut.GlobalMemberInfos,
							ReflectionOut.GlobalRemap,
							ReflectionOut.GlobalArrays);
					}
				}
			}
		}
	}

	{
		ReflectionBindings.GatherOutputAttributes(Reflection);
		for (SpvReflectInterfaceVariable* Var : ReflectionBindings.OutputAttributes)
		{
			if (Var->storage_class == SpvStorageClassOutput && Var->built_in == -1 && !CrossCompiler::FShaderConductorContext::IsIntermediateSpirvOutputVariable(Var->name))
			{
				FString TypeQualifier;

				auto const type = *Var->type_description;
				uint32_t masked_type = type.type_flags & 0xF;

				switch (masked_type) {
				default: checkf(false, TEXT("unsupported component type %d"), masked_type); break;
				case SPV_REFLECT_TYPE_FLAG_BOOL: TypeQualifier = TEXT("b"); break;
				case SPV_REFLECT_TYPE_FLAG_INT: TypeQualifier = (type.traits.numeric.scalar.signedness ? TEXT("i") : TEXT("u")); break;
				case SPV_REFLECT_TYPE_FLAG_FLOAT: TypeQualifier = (type.traits.numeric.scalar.width == 32 ? TEXT("f") : TEXT("h")); break;
				}

				if (type.type_flags & SPV_REFLECT_TYPE_FLAG_MATRIX)
				{
					TypeQualifier += FString::Printf(TEXT("%d%d"), type.traits.numeric.matrix.row_count, type.traits.numeric.matrix.column_count);
				}
				else if (type.type_flags & SPV_REFLECT_TYPE_FLAG_VECTOR)
				{
					TypeQualifier += FString::Printf(TEXT("%d"), type.traits.numeric.vector.component_count);
				}
				else
				{
					TypeQualifier += TEXT("1");
				}

				FString Name = ANSI_TO_TCHAR(Var->name);
				Name.ReplaceInline(TEXT("."), TEXT("_"));

				if (Frequency == SF_Pixel && strstr(Var->name, "SV_Target"))
				{
					ReflectionOut.OutputVarNames.Add(Name);
					CCHeaderWriter.WriteOutputAttribute(TEXT("out_Target"), *TypeQualifier, Var->location, /*bLocationPrefix:*/ true, /*bLocationSuffix:*/ true);
				}
				else
				{
					CCHeaderWriter.WriteOutputAttribute(*Name, *TypeQualifier, Var->location, /*bLocationPrefix:*/ true, /*bLocationSuffix:*/ false);
				}
			}
		}
	}

	{
		uint32 AssignedInputs = 0;

		ReflectionBindings.GatherInputAttributes(Reflection);
		for (SpvReflectInterfaceVariable* Var : ReflectionBindings.InputAttributes)
		{
			if (Var->storage_class == SpvStorageClassInput && Var->built_in == -1 && Frequency == SF_Vertex)
			{
				unsigned Location = Var->location;
				unsigned SemanticIndex = Location;
				check(Var->semantic);
				unsigned i = (unsigned)strlen(Var->semantic);
				check(i);
				while (isdigit((unsigned char)(Var->semantic[i - 1])))
				{
					i--;
				}
				if (i < strlen(Var->semantic))
				{
					SemanticIndex = (unsigned)atoi(Var->semantic + i);
					if (Location != SemanticIndex)
					{
						Location = SemanticIndex;
					}
				}

				while ((1 << Location) & AssignedInputs)
				{
					Location++;
				}

				if (Location != Var->location)
				{
					SPVRResult = Reflection.ChangeInputVariableLocation(Var, Location);
					check(SPVRResult == SPV_REFLECT_RESULT_SUCCESS);
				}

				uint32 ArrayCount = 1;
				for (uint32 Dim = 0; Dim < Var->array.dims_count; Dim++)
				{
					ArrayCount *= Var->array.dims[Dim];
				}

				FString TypeQualifier;

				auto const type = *Var->type_description;
				uint32_t masked_type = type.type_flags & 0xF;

				switch (masked_type) {
				default: checkf(false, TEXT("unsupported component type %d"), masked_type); break;
				case SPV_REFLECT_TYPE_FLAG_BOOL: TypeQualifier = TEXT("b"); break;
				case SPV_REFLECT_TYPE_FLAG_INT: TypeQualifier = (type.traits.numeric.scalar.signedness ? TEXT("i") : TEXT("u")); break;
				case SPV_REFLECT_TYPE_FLAG_FLOAT: TypeQualifier = (type.traits.numeric.scalar.width == 32 ? TEXT("f") : TEXT("h")); break;
				}

				if (type.type_flags & SPV_REFLECT_TYPE_FLAG_MATRIX)
				{
					TypeQualifier += FString::Printf(TEXT("%d%d"), type.traits.numeric.matrix.row_count, type.traits.numeric.matrix.column_count);
				}
				else if (type.type_flags & SPV_REFLECT_TYPE_FLAG_VECTOR)
				{
					TypeQualifier += FString::Printf(TEXT("%d"), type.traits.numeric.vector.component_count);
				}
				else
				{
					TypeQualifier += TEXT("1");
				}

				for (uint32 j = 0; j < ArrayCount; j++)
				{
					AssignedInputs |= (1 << (Location + j));
				}

				FString Name = ANSI_TO_TCHAR(Var->name);
				Name.ReplaceInline(TEXT("."), TEXT("_"));
				ReflectionOut.InputVarNames.Add(Name);
				CCHeaderWriter.WriteInputAttribute(*Name, *TypeQualifier, Location, /*bLocationPrefix:*/ true, /*bLocationSuffix:*/ false);
			}
		}
	}
}

static void ConvertToEmulatedUBs(std::string& GlslSource, const ReflectionData& ReflectData, const EShaderFrequency Frequency)
{
	const ANSICHAR* FrequencyPrefix = GetFrequencyPrefix(Frequency);
	
	for (const auto& PackedUBNamePair : ReflectData.PackedUBNames)
	{
		bool bIsLayout = true;
		std::string UBSearchString = "layout\\(.*\\) uniform type_" + PackedUBNamePair.Value + "$";

		std::smatch RegexMatch;
		if (std::regex_search(GlslSource, RegexMatch, std::regex(UBSearchString)) && !RegexMatch.empty())
		{
			UBSearchString = RegexMatch[0];
		}

		size_t UBPos = GlslSource.find(UBSearchString);

		if (UBPos == std::string::npos)
		{
			UBSearchString = "struct type_" + PackedUBNamePair.Value;
			UBPos = GlslSource.find(UBSearchString);
			bIsLayout = false;
		}

		if (UBPos != std::string::npos)
		{
			std::string UBEndSearchString;
			if (bIsLayout)
			{
				UBEndSearchString = "} " + PackedUBNamePair.Value + ";";
			}
			else
			{
				UBEndSearchString = "uniform type_" + PackedUBNamePair.Value + " " + PackedUBNamePair.Value + ";";
			}

			size_t UBEndPos = GlslSource.find(UBEndSearchString);

			if (UBEndPos != std::string::npos)
			{
				GlslSource.erase(UBPos, UBEndPos - UBPos + UBEndSearchString.length());

				size_t UBVarPos = 0;

				for (const PackedUBMemberInfo& MemberInfo : ReflectData.PackedUBMemberInfos[PackedUBNamePair.Key])
				{
					std::string UBVarString = PackedUBNamePair.Value + "." + MemberInfo.Name;

					UBVarPos = GlslSource.find(UBVarString);
					while (UBVarPos != std::string::npos)
					{
						GlslSource.erase(UBVarPos, PackedUBNamePair.Value.length() + 1);
						GlslSource.replace(UBVarPos, MemberInfo.Name.size(), MemberInfo.SanitizedName);

						for (std::string const& SearchString : ReflectData.PackedUBArrays[PackedUBNamePair.Key])
						{
							// This is done in case the current SearchString is a substring of another
							std::string SearchStringWithBraces = SearchString + "[";
							if (!GlslSource.compare(UBVarPos, SearchStringWithBraces.length(), SearchStringWithBraces))
							{
								GlslSource.replace(UBVarPos + SearchString.length(), 1, "(");

								size_t ClosingBrace = GlslSource.find("]", UBVarPos + SearchString.length());
								if (ClosingBrace != std::string::npos)
									GlslSource.replace(ClosingBrace, 1, ")");
							}
						}

						UBVarPos = GlslSource.find(UBVarString);
					}
				}

				for (auto const& Pair : ReflectData.PackedUBOffsets[PackedUBNamePair.Key])
				{
					if (Pair.Value > 0)
					{
						std::string NewUniforms;
						std::string UniformPrefix = FrequencyPrefix + std::string("c") + std::to_string(PackedUBNamePair.Key);
						GetPackedUniformString(NewUniforms, UniformPrefix, Pair.Key, Pair.Value);
						GlslSource.insert(UBPos, NewUniforms);
					}
				}

				for (std::string const& Define : ReflectData.PackedUBRemap[PackedUBNamePair.Key])
				{
					GlslSource.insert(UBPos, Define);
				}
			}
		}
	}
}

const ANSICHAR* GlslFrameBufferExtensions =
"\n\n#ifdef UE_MRT_FRAMEBUFFER_FETCH\n"
"\t#extension GL_EXT_shader_framebuffer_fetch : enable\n"
"\t#define FBF_STORAGE_QUALIFIER inout\n"
"#elif defined(GL_ARM_shader_framebuffer_fetch)\n"
"\t#extension GL_ARM_shader_framebuffer_fetch : enable\n"
"#endif\n"
"#ifdef GL_ARM_shader_framebuffer_fetch_depth_stencil\n"
"\t#extension GL_ARM_shader_framebuffer_fetch_depth_stencil : enable\n"
"#endif\n"
"// end extensions";

// GLSL framebuffer macro definitions. Used to patch GLSL output source.
const ANSICHAR* GlslFrameBufferDefines =
"\n\n#ifdef UE_MRT_FRAMEBUFFER_FETCH\n"
"\t#define _Globals_ARM_shader_framebuffer_fetch 0u\n"
"\t#define FRAME_BUFFERFETCH_STORAGE_QUALIFIER inout\n"
"\t#define _Globals_gl_FragColor out_var_SV_Target0\n"
"\t#define _Globals_gl_LastFragColorARM vec4(0.0, 0.0, 0.0, 0.0)\n"
"#elif defined(GL_ARM_shader_framebuffer_fetch)\n"
"\t#define _Globals_ARM_shader_framebuffer_fetch 1u\n"
"\t#define FRAME_BUFFERFETCH_STORAGE_QUALIFIER out\n"
"\t#define _Globals_gl_FragColor vec4(0.0, 0.0, 0.0, 0.0)\n"
"\t#define _Globals_gl_LastFragColorARM gl_LastFragColorARM\n"
"#else\n"
"\t#define FRAME_BUFFERFETCH_STORAGE_QUALIFIER out\n"
"\t#define _Globals_ARM_shader_framebuffer_fetch 0u\n"
"\t#define _Globals_gl_FragColor vec4(0.0, 0.0, 0.0, 0.0)\n"
"\t#define _Globals_gl_LastFragColorARM vec4(0.0, 0.0, 0.0, 0.0)\n"
"#endif\n"
"#ifdef GL_ARM_shader_framebuffer_fetch_depth_stencil\n"
"\t#define _Globals_ARM_shader_framebuffer_fetch_depth_stencil 1u\n"
"#else\n"
"\t#define _Globals_ARM_shader_framebuffer_fetch_depth_stencil 0u\n"
"#endif\n";

struct GLSLCompileParameters
{
	CrossCompiler::FShaderConductorContext* CompilerContext;
	CrossCompiler::FHlslccHeaderWriter* CCHeaderWriter;
	CrossCompiler::FShaderConductorTarget* TargetDesc;
	CrossCompiler::FShaderConductorOptions* Options;

	FShaderCompilerOutput* Output;
	TArray<uint32> SpirvData;

	EShaderFrequency Frequency;
	const ANSICHAR* SPIRV_DummySamplerName;
};

bool GenerateGlslShader(std::string& OutString, GLSLCompileParameters& GLSLCompileParams, ReflectionData& ReflectData, bool bWriteToCCHeader, bool bIsDeferred, bool bEmulatedUBs)
{
	const bool bGlslSourceCompileSucceeded = GLSLCompileParams.CompilerContext->CompileSpirvToSourceBuffer(
		*GLSLCompileParams.Options, *GLSLCompileParams.TargetDesc, GLSLCompileParams.SpirvData.GetData(), GLSLCompileParams.SpirvData.Num() * sizeof(uint32),
		[&OutString](const void* Data, uint32 Size)
		{
			OutString = std::string(reinterpret_cast<const ANSICHAR*>(Data), Size);
		}
	);

	if (!bGlslSourceCompileSucceeded)
	{
		GLSLCompileParams.CompilerContext->FlushErrors(GLSLCompileParams.Output->Errors);
		return false;
	}

	const ANSICHAR* FrequencyPrefix = GetFrequencyPrefix(GLSLCompileParams.Frequency);

	std::string LayoutString = "#extension ";
	size_t LayoutPos = OutString.find(LayoutString);
	if (LayoutPos != std::string::npos)
	{
		for (FString Name : ReflectData.InputVarNames)
		{
			std::string DefineString = "#define ";
			DefineString += TCHAR_TO_ANSI(*Name);
			DefineString += " ";
			DefineString += TCHAR_TO_ANSI(*Name.Replace(TEXT("in_var_"), TEXT("in_")));
			DefineString += "\n";

			OutString.insert(LayoutPos, DefineString);
		}
		for (FString Name : ReflectData.OutputVarNames)
		{
			std::string DefineString = "#define ";
			DefineString += TCHAR_TO_ANSI(*Name);
			DefineString += " ";
			DefineString += TCHAR_TO_ANSI(*Name.Replace(TEXT("out_var_SV_"), TEXT("out_")));
			DefineString += "\n";

			OutString.insert(LayoutPos, DefineString);
		}
	}

	// Perform FBF replacements
	if (GLSLCompileParams.Frequency == SF_Pixel)
	{
		size_t MainPos = OutString.find("#version 320 es");

		// Fallback if the shader is 310 
		if (MainPos == std::string::npos)
		{
			MainPos = OutString.find("#version 310 es");
		}

		if (MainPos != std::string::npos)
		{
			MainPos += strlen("#version 320 es");
		}

		// Framebuffer Depth Fetch
		{
			std::string FBFString = "_Globals.ARM_shader_framebuffer_fetch";
			std::string FBFReplaceString = "_Globals_ARM_shader_framebuffer_fetch";

			bool UsesFramebufferFetch = OutString.find(FBFString) != std::string::npos;

			if (UsesFramebufferFetch)
			{
				OutString.insert(MainPos, GlslFrameBufferDefines);
				OutString.insert(MainPos, GlslFrameBufferExtensions);

				std::string FBDFString = "_Globals.ARM_shader_framebuffer_fetch_depth_stencil";
				std::string FBDFReplaceString = "1u";

				bool UsesFramebufferDepthFetch = OutString.find(FBDFString) != std::string::npos;
				if (UsesFramebufferDepthFetch)
				{
					size_t FramebufferDepthFetchPos = OutString.find(FBDFString);

					OutString.erase(FramebufferDepthFetchPos, FBDFString.length());
					OutString.insert(FramebufferDepthFetchPos, FBDFReplaceString);

					std::string LastFragDepthARMString = "_Globals._RESERVED_IDENTIFIER_FIXUP_gl_LastFragDepthARM";
					std::string LastFragDepthARMReplaceString = "GLFetchDepthBuffer()";

					size_t LastFragDepthARMStringPos = OutString.find(LastFragDepthARMString);

					while (LastFragDepthARMStringPos != std::string::npos)
					{
						OutString.erase(LastFragDepthARMStringPos, LastFragDepthARMString.length());
						OutString.insert(LastFragDepthARMStringPos, LastFragDepthARMReplaceString);

						LastFragDepthARMStringPos = OutString.find(LastFragDepthARMString);
					}

					MainPos = OutString.find("void main()");

					// Add support for framebuffer fetch depth when ARM extension is not supported
					if (MainPos != std::string::npos)
					{
						std::string DepthBufferIndex = bIsDeferred ? "4" : "1";
						std::string DepthBufferOutVarString = "out_var_SV_Target" + DepthBufferIndex;

						// Insert function declaration to handle retrieving depth
						OutString.insert(MainPos, "float GLFetchDepthBuffer()\n"
							"{\n"
							"\t#if defined(GL_ARM_shader_framebuffer_fetch_depth_stencil)\n"
							"\treturn gl_LastFragDepthARM;\n"
							"\t#elif defined(GL_EXT_shader_framebuffer_fetch)\n"
							"\treturn " + DepthBufferOutVarString + ".x;\n"
							"\t#else\n"
							"\treturn 0.0f;\n"
							"\t#endif\n"
							"}\n");


						// If SceneDepthAux is not declared then declare it, otherwise modify so that we only enable it on devices that don't support
						// GL_ARM_shader_framebuffer_fetch_depth_stencil and do support GL_EXT_shader_framebuffer_fetch
						size_t DepthBufferOutVarPos = OutString.find(DepthBufferOutVarString + ";");

						std::string DepthBufferDeclString = "layout(location = " + DepthBufferIndex + ") inout highp vec4 " + DepthBufferOutVarString + ";\n";
						std::string DepthBufferOutString = "\n#if !defined(GL_ARM_shader_framebuffer_fetch_depth_stencil) && defined(GL_EXT_shader_framebuffer_fetch)\n" +
							DepthBufferDeclString +
							"#endif\n";

						// If we cannot find a declararation of out_var_SV_Target(n) in the shader, insert one
						if (DepthBufferOutVarPos == std::string::npos)
						{
							OutString.insert(MainPos, DepthBufferOutString);
						}
						else
						{
							// If we have a declaration, replace with one that will be stripped if GL_ARM_shader_framebuffer_fetch_depth_stencil is enabled
							size_t StringStartPos = OutString.rfind("layout", DepthBufferOutVarPos - 1);
							size_t StringEndPos = OutString.find(";", StringStartPos);

							OutString.erase(StringStartPos, (StringEndPos + 1) - StringStartPos);
							OutString.insert(StringStartPos, DepthBufferOutString);
						}

						// Make SceneDepthAux assignment conditional
						// We only need to write the depth when we don't support GL_ARM_shader_framebuffer_fetch_depth_stencil
						std::string DepthBufferAssignment = DepthBufferOutVarString + " =";
						size_t DepthBufferAssignmentPos = OutString.find(DepthBufferAssignment);

						if (DepthBufferAssignmentPos != std::string::npos)
						{
							size_t LineEnd = OutString.find_first_of(";", DepthBufferAssignmentPos);
							uint32_t AssignmentValueStart = DepthBufferAssignmentPos + DepthBufferAssignment.size();
							std::string AssignmentValue = OutString.substr(AssignmentValueStart + 1, LineEnd - AssignmentValueStart);

							if (LineEnd != std::string::npos)
							{
								OutString.erase(DepthBufferAssignmentPos, LineEnd + 1 - DepthBufferAssignmentPos);
								OutString.insert(DepthBufferAssignmentPos, std::string("#if !defined(GL_ARM_shader_framebuffer_fetch_depth_stencil) && defined(GL_EXT_shader_framebuffer_fetch)\n") + DepthBufferAssignment + AssignmentValue + std::string("\n#endif\n"));
							}
						}
					}
				} // UsesFramebufferDepthFetch
			}

			// check again, before it might have been a match for ARM_shader_framebuffer_fetch_depth_stencil
			UsesFramebufferFetch = GLSLCompileParams.Frequency == SF_Pixel && OutString.find(FBFString) != std::string::npos;
			if (UsesFramebufferFetch)
			{
				std::string ReservedIdentifierFixupString = "_RESERVED_IDENTIFIER_FIXUP_";
				size_t ReservedIdentifierFixupStringPos = OutString.find(ReservedIdentifierFixupString);

				while (ReservedIdentifierFixupStringPos != std::string::npos)
				{
					OutString.erase(ReservedIdentifierFixupStringPos, ReservedIdentifierFixupString.length());
					ReservedIdentifierFixupStringPos = OutString.find(ReservedIdentifierFixupString);
				}

				std::string Target0String = "layout(location = 0) out ";
				std::string Target0ReplaceString = "layout(location = 0) FRAME_BUFFERFETCH_STORAGE_QUALIFIER ";
				size_t Target0StringPos = OutString.find(Target0String);
				if (Target0StringPos != std::string::npos)
				{
					OutString.erase(Target0StringPos, Target0String.length());
					OutString.insert(Target0StringPos, Target0ReplaceString);
				}
			}
		}
	}


	// If we are rendering deferred, then we only need SceneDepthAux on devices that don't support framebuffer fetch depth
	if (bIsDeferred)
	{
		std::string SceneDepthAux = "layout(location = 4) out highp float out_var_SV_Target4;";
		size_t SceneDepthAuxPos = OutString.find(SceneDepthAux);
		if (SceneDepthAuxPos != std::string::npos)
		{
			OutString.insert(SceneDepthAuxPos + SceneDepthAux.size(), "\n#endif\n");
			OutString.insert(SceneDepthAuxPos, "\n#ifndef GL_ARM_shader_framebuffer_fetch_depth_stencil\n");

			std::string SceneDepthAuxAssignment = "out_var_SV_Target4 =";
			size_t SceneDepthAuxAssignmentPos = OutString.find(SceneDepthAuxAssignment);

			if (SceneDepthAuxAssignmentPos != std::string::npos)
			{
				size_t LineEnd = OutString.find_first_of(";", SceneDepthAuxAssignmentPos);
				uint32_t AssignmentValueStart = SceneDepthAuxAssignmentPos + SceneDepthAuxAssignment.size();
				std::string AssignmentValue = OutString.substr(AssignmentValueStart + 1, LineEnd - AssignmentValueStart);

				if (LineEnd != std::string::npos)
				{
					OutString.erase(SceneDepthAuxAssignmentPos, LineEnd + 1 - SceneDepthAuxAssignmentPos);
					OutString.insert(SceneDepthAuxAssignmentPos, std::string("#ifndef GL_ARM_shader_framebuffer_fetch_depth_stencil\n") + SceneDepthAuxAssignment + AssignmentValue + std::string("\n#endif\n"));
				}
			}
		}
	}

	// Fixup packed globals
	{
		bool bIsLayout = true;

		std::string GlobalsSearchString = "layout\\(.*\\) uniform type_Globals$";

		std::smatch RegexMatch;
		if (std::regex_search(OutString, RegexMatch, std::regex(GlobalsSearchString)) && !RegexMatch.empty())
		{
			GlobalsSearchString = RegexMatch[0];
		}

		size_t GlobalPos = OutString.find(GlobalsSearchString);

		if (GlobalPos == std::string::npos)
		{
			GlobalsSearchString = "struct type_Globals";
			GlobalPos = OutString.find(GlobalsSearchString);
			bIsLayout = false;
		}

		if (GlobalPos != std::string::npos)
		{
			std::string GlobalsEndSearchString;
			if (bIsLayout)
			{
				GlobalsEndSearchString = "} _Globals;";
			}
			else
			{
				GlobalsEndSearchString = "uniform type_Globals _Globals;";
			}

			size_t GlobalEndPos = OutString.find(GlobalsEndSearchString);

			if (GlobalEndPos != std::string::npos)
			{
				OutString.erase(GlobalPos, GlobalEndPos - GlobalPos + GlobalsEndSearchString.length());

				std::string GlobalVarString = "_Globals.";
				size_t GlobalVarPos = 0;
				do
				{
					GlobalVarPos = OutString.find(GlobalVarString, GlobalVarPos);
					if (GlobalVarPos != std::string::npos)
					{
						OutString.replace(GlobalVarPos, GlobalVarString.length(), "_Globals_");
						for (std::string const& SearchString : ReflectData.GlobalArrays)
						{
							// This is done in case the current SearchString is a substring of another
							std::string SearchStringWithBraces = SearchString + "[";
							if (!OutString.compare(GlobalVarPos, SearchStringWithBraces.length(), SearchStringWithBraces))
							{
								OutString.replace(GlobalVarPos + SearchString.length(), 1, "(");

								size_t ClosingBrace = OutString.find("]", GlobalVarPos + SearchString.length());
								if (ClosingBrace != std::string::npos)
									OutString.replace(ClosingBrace, 1, ")");
							}
						}
					}
				} while (GlobalVarPos != std::string::npos);

				for (auto const& Pair : ReflectData.GlobalOffsets)
				{
					if (Pair.Value > 0)
					{
						std::string NewUniforms;
						GetPackedUniformString(NewUniforms, FrequencyPrefix + std::string("u"), Pair.Key, Pair.Value);
						OutString.insert(GlobalPos, NewUniforms);
					}
				}

				for (std::string const& Define : ReflectData.GlobalRemap)
				{
					OutString.insert(GlobalPos, Define);
				}

				bool UsesFramebufferFetch = GLSLCompileParams.Frequency == SF_Pixel && OutString.find("_Globals.ARM_shader_framebuffer_fetch") != std::string::npos;

				if (UsesFramebufferFetch)
				{
					size_t OutColor = OutString.find("0) out ");
					if (OutColor != std::string::npos)
						OutString.replace(OutColor, 7, "0) FRAME_BUFFERFETCH_STORAGE_QUALIFIER ");
				}
			}
		}
	}

	if (bEmulatedUBs)
	{
		ConvertToEmulatedUBs(OutString, ReflectData, GLSLCompileParams.Frequency);
	}

	const size_t GlslSourceLen = OutString.length();
	if (GlslSourceLen > 0)
	{
		uint32 TextureIndex = 0;
		for (const FString& Texture : ReflectData.Textures)
		{
			const size_t SamplerPos = OutString.find("\nuniform ");

			TArray<FString> UsedSamplers;
			FString SamplerString;
			for (const FString& Sampler : ReflectData.Samplers)
			{
				std::string SamplerName = "SPIRV_Cross_Combined";
				SamplerName += TCHAR_TO_ANSI(*(Texture + Sampler));
				std::string SamplerNameDecl = SamplerName + ";";
				size_t FindCombinedSampler = OutString.find(SamplerNameDecl.c_str());

				if (FindCombinedSampler != std::string::npos)
				{
					checkf(SamplerPos != std::string::npos, TEXT("Generated GLSL shader is expected to have combined sampler '%s:%s' but no appropriate 'uniform' declaration could be found"), *Texture, *Sampler);

					uint32 NewIndex = TextureIndex + UsedSamplers.Num();
					std::string NewDefine = "#define ";
					NewDefine += SamplerName;
					NewDefine += " ";
					NewDefine += FrequencyPrefix;
					NewDefine += "s";
					NewDefine += std::to_string(NewIndex);
					NewDefine += "\n";
					OutString.insert(SamplerPos + 1, NewDefine);

					// Do not add an entry for the dummy sampler as it will throw errors in the runtime checks
					if (Sampler != GLSLCompileParams.SPIRV_DummySamplerName)
					{
						UsedSamplers.Add(Sampler);
					}

					SamplerString += FString::Printf(TEXT("%s%s"), SamplerString.Len() ? TEXT(",") : TEXT(""), *Sampler);
				}
			}

			// Rename texture buffers
			std::string SamplerBufferName = std::string("samplerBuffer ") + TCHAR_TO_ANSI(*Texture);
			size_t SamplerBufferPos = OutString.find(SamplerBufferName);

			if (SamplerBufferPos != std::string::npos)
			{
				std::string NewSamplerBufferName = std::string(FrequencyPrefix) + "s" + std::to_string(TextureIndex);

				OutString.erase(SamplerBufferPos + SamplerBufferName.size() - Texture.Len(), Texture.Len());
				OutString.insert(SamplerBufferPos + SamplerBufferName.size() - Texture.Len(), NewSamplerBufferName);

				std::string SamplerTexFetchExpression = std::string("texelFetch\\(\\s?") + TCHAR_TO_ANSI(*Texture) + "\\s?,";
				std::regex SamplerTexFetchPattern(SamplerTexFetchExpression);
				OutString = std::regex_replace(OutString, SamplerTexFetchPattern, std::string("texelFetch(") + NewSamplerBufferName + ",");
			}

			const uint32 SamplerCount = FMath::Max(1, UsedSamplers.Num());
			if (bWriteToCCHeader)
			{
				GLSLCompileParams.CCHeaderWriter->WriteSRV(*Texture, TextureIndex, SamplerCount, UsedSamplers);
			}
			TextureIndex += SamplerCount;
		}

		// UAVS, rename as ci0 format
		uint32_t UAVIndex = 0;
		for (const std::string& UAV : ReflectData.UAVs)
		{
			std::string NewUAVName = FrequencyPrefix + std::string("i") + std::to_string(UAVIndex);

			// Find instances of UAVs
			std::string RegexExpression = "\\b" + UAV + "\\b";
			std::regex RegexExpressionPattern(RegexExpression);

			OutString = std::regex_replace(OutString, RegexExpressionPattern, NewUAVName);

			UAVIndex++;
		}

		// StructuredBuffers, rename as ci0 format
		// Presort the rename order to avoid search collisions for names like MyBuffer and MyBufferWithSuffix.
		using FIndexNamePair = TPair<int32, const std::string&>;
		TArray<FIndexNamePair> SortedBuffers;
		SortedBuffers.Reserve(ReflectData.StructuredBuffers.Num());
		for (const std::string& SBuffer : ReflectData.StructuredBuffers)
		{
			SortedBuffers.Add(FIndexNamePair(UAVIndex++, SBuffer));
		}
		Algo::Sort(SortedBuffers, [](const FIndexNamePair& A, const FIndexNamePair& B){ return A.Value > B.Value; });
		
		for (const FIndexNamePair& SortedEntry : SortedBuffers)
		{
			const std::string& SBuffer = SortedEntry.Value;
			std::string NewSBufferName = FrequencyPrefix + std::string("i") + std::to_string(SortedEntry.Key);
			std::string NewSBufferVarName = NewSBufferName + std::string("_VAR");

			std::string SearchString = "} " + SBuffer;
			size_t SBufferPos = OutString.find(SearchString);

			size_t SBufferEndPos = OutString.find(";", SBufferPos);

			std::string ReplacementSubStr = OutString.substr(SBufferPos + 2, SBufferEndPos - SBufferPos - 2);
			OutString.erase(SBufferPos + 1, SBufferEndPos - SBufferPos - 1);

			const std::string SBufferData = "_m0";
			size_t SBufferDataPos = OutString.rfind(SBufferData, SBufferPos);
			OutString.replace(SBufferDataPos, SBufferData.size(), NewSBufferName);

			const std::string BufferString = " buffer ";
			size_t BufferNamePos = OutString.rfind(BufferString, SBufferPos);
			size_t BufferLineEndPos = OutString.find("\n", BufferNamePos);
			size_t ReplacePos = BufferNamePos + BufferString.size();

			OutString.replace(ReplacePos, BufferLineEndPos - ReplacePos, NewSBufferVarName);

			// Replace the usage of StructuredBuffer with new name
			size_t CurPos = OutString.find(ReplacementSubStr + ".");
			while (CurPos != std::string::npos)
			{
				// Offset by 4 to account for ._m0
				OutString.replace(CurPos, ReplacementSubStr.size() + 4, NewSBufferName);
				CurPos = OutString.find(ReplacementSubStr + ".");
			}
		}

		for (const auto& pair : ReflectData.UniformVarNames)
		{
			std::string OldUniformTypeName = "uniform type_" + pair.first + " " + pair.first + ";";

			size_t UniformTypePos = OutString.find(OldUniformTypeName);

			if (UniformTypePos != std::string::npos)
			{
				OutString.erase(UniformTypePos, OldUniformTypeName.length());
			}

			// Replace struct type with layout, for compatibility 
			std::string OldStructTypeName = "struct type_" + pair.first + "\n";
			std::string NewStructTypeName = "layout(std140) uniform " + pair.second + "\n";

			size_t StructTypePos = OutString.find(OldStructTypeName);

			if (StructTypePos != std::string::npos)
			{
				OutString.erase(StructTypePos, OldStructTypeName.length());
				OutString.insert(StructTypePos, NewStructTypeName);
			}

			// Append the uniform buffer name i.e. pb0 to ensure variable names are unique across shaders
			std::vector<std::string>& MemberNames = ReflectData.UniformVarMemberNames[pair.first];
			for (const std::string& uniform_member_name : MemberNames)
			{
				std::string OldVarName = uniform_member_name;
				std::string NewVarName = uniform_member_name + pair.second;

				size_t OldVarNamePos = OutString.find(OldVarName);

				OutString.erase(OldVarNamePos, OldVarName.length());
				OutString.insert(OldVarNamePos, NewVarName);
			}

			// Sort the member names by length so that we don't accidently replace a partial string
			std::sort(MemberNames.begin(), MemberNames.end(), [](const std::string& a, const std::string& b) {
				return a.length() > b.length();
				});

			// Replace uniform member names in the glsl
			for (const std::string& uniform_member_name : MemberNames)
			{
				std::string OldUniformBufferAccessor = pair.first + "." + uniform_member_name;
				std::string NewUniformBufferAccessor = uniform_member_name + pair.second;

				size_t OldAccessorPos = OutString.find(OldUniformBufferAccessor);
				while (OldAccessorPos != std::string::npos)
				{
					OutString.erase(OldAccessorPos, OldUniformBufferAccessor.length());
					OutString.insert(OldAccessorPos, NewUniformBufferAccessor);

					OldAccessorPos = OutString.find(OldUniformBufferAccessor);
				}
			}

			// Rename the padding variables to stop conflicts between shaders
			std::string OldPaddingName = "type_" + pair.first + "_pad";
			std::string NewPaddingName = pair.second + "_pad";

			size_t OldPaddingPos = OutString.find(OldPaddingName);
			while (OldPaddingPos != std::string::npos)
			{
				OutString.erase(OldPaddingPos, OldPaddingName.length());
				OutString.insert(OldPaddingPos, NewPaddingName);

				OldPaddingPos = OutString.find(OldPaddingName);
			}
		}
	}

	return true;
}

enum EDecalBlendFlags
{
	DecalOut_MRT0  	= 1,
	DecalOut_MRT1  	= 1 << 1,
	DecalOut_MRT2  	= 1 << 2,
	DecalOut_MRT3  	= 1 << 3,
	Translucent		= 1 << 4,
	AlphaComposite  = 1 << 5,
	Modulate		= 1 << 6,
};

uint32_t GetDecalBlendFlags(const FShaderCompilerInput& Input)
{
	uint32_t Flags = 0;

	if (Input.Environment.GetCompileArgument(TEXT("DECAL_OUT_MRT0"), false))
	{
		Flags |= EDecalBlendFlags::DecalOut_MRT0;
	}
	if (Input.Environment.GetCompileArgument(TEXT("DECAL_OUT_MRT1"), false))
	{
		Flags |= EDecalBlendFlags::DecalOut_MRT1;
	}
	if (Input.Environment.GetCompileArgument(TEXT("DECAL_OUT_MRT2"), false))
	{
		Flags |= EDecalBlendFlags::DecalOut_MRT2;
	}
	if (Input.Environment.GetCompileArgument(TEXT("DECAL_OUT_MRT3"), false))
	{
		Flags |= EDecalBlendFlags::DecalOut_MRT3;
	}

	if (!Flags)
	{
		return 0;
	}

	if (Input.Environment.GetCompileArgument(TEXT("MATERIALBLENDING_ALPHACOMPOSITE"), false))
	{
		Flags |= EDecalBlendFlags::AlphaComposite;
	}
	else if (Input.Environment.GetCompileArgument(TEXT("MATERIALBLENDING_MODULATE"), false))
	{
		Flags |= EDecalBlendFlags::Modulate;
	}
	else if (Input.Environment.GetCompileArgument(TEXT("MATERIALBLENDING_TRANSLUCENT"), false))
	{
		Flags |= EDecalBlendFlags::Translucent;
	}

	return Flags;
}

enum class EDecalBlendFunction
{
	SrcAlpha_One,
	SrcAlpha_InvSrcAlpha,
	DstColor_InvSrcAlpha,
	One_InvSrcAlpha,
	None
};

//																	Emissive,							Normal,										Metallic\Specular\Roughness,				BaseColor
static const EDecalBlendFunction TranslucentBlendFunctions[]	= { EDecalBlendFunction::SrcAlpha_One,	EDecalBlendFunction::SrcAlpha_InvSrcAlpha,	EDecalBlendFunction::SrcAlpha_InvSrcAlpha,  EDecalBlendFunction::SrcAlpha_InvSrcAlpha };
static const EDecalBlendFunction AlphaCompositeBlendFunctions[] = { EDecalBlendFunction::SrcAlpha_One,	EDecalBlendFunction::None,					EDecalBlendFunction::One_InvSrcAlpha,		EDecalBlendFunction::One_InvSrcAlpha };
static const EDecalBlendFunction ModulateBlendFunctions[]		= { EDecalBlendFunction::SrcAlpha_One,	EDecalBlendFunction::SrcAlpha_InvSrcAlpha,	EDecalBlendFunction::SrcAlpha_InvSrcAlpha,  EDecalBlendFunction::DstColor_InvSrcAlpha };

void GetDecalBlendFunctionString(const EDecalBlendFunction BlendFunction, const std::string& AttachmentString, const std::string& TempOutputName, std::string& OutputString)
{
	switch (BlendFunction)
	{
		case EDecalBlendFunction::SrcAlpha_One:
		{
			OutputString = AttachmentString + ".rgb = " + TempOutputName + ".a * " + TempOutputName + ".rgb + " + AttachmentString + ".rgb";
			break;
		}
		case EDecalBlendFunction::SrcAlpha_InvSrcAlpha:
		{
			OutputString = AttachmentString + ".rgb = " + TempOutputName + ".a * " + TempOutputName + ".rgb + (1.0 - " + TempOutputName + ".a) * " + AttachmentString + ".rgb";
			break;
		}
		case EDecalBlendFunction::DstColor_InvSrcAlpha:
		{
			OutputString = AttachmentString + ".rgb = " + TempOutputName + ".rgb * " + AttachmentString + ".rgb + (1.0 - " + TempOutputName + ".a) * " + AttachmentString + ".rgb";
			break;
		}
		case EDecalBlendFunction::One_InvSrcAlpha:
		{
			OutputString = AttachmentString + ".rgb = " + TempOutputName + ".rgb + (1.0 - " + TempOutputName + ".a) * " + AttachmentString + ".rgb";
			break;
		}
		case EDecalBlendFunction::None:
		{
			OutputString = "";
			break;
		}
	}
}

void ModifyDecalBlending(std::string& SourceString, const uint32_t BlendFlags)
{
	for (uint32_t i = 0; i < 4; ++i)
	{
		std::string OutIndex = std::to_string(i);
		std::string AttachmentString = "GENERATED_SubpassFetchAttachment" + OutIndex;
		std::string AttachmentAssignmentString = AttachmentString + " =";

		size_t AssignmentPos = SourceString.find(AttachmentAssignmentString);
		if (AssignmentPos != std::string::npos)
		{
			size_t AssignmentEndPos = AssignmentPos + AttachmentAssignmentString.size();
			size_t StringEndPos = SourceString.find(";", AssignmentEndPos);

			std::string TempOutputName = "TempMulOut" + OutIndex;
			std::string TempOut = "highp vec4 " + TempOutputName + " = " + SourceString.substr(AssignmentEndPos, StringEndPos - AssignmentEndPos) + "; \n";

			SourceString.erase(AssignmentPos, StringEndPos - AssignmentPos);

			if (1 << i & BlendFlags)
			{
				std::string BlendFnString;

				if (BlendFlags & EDecalBlendFlags::Translucent)
				{
					GetDecalBlendFunctionString(TranslucentBlendFunctions[i], AttachmentString, TempOutputName, BlendFnString);
				}
				if (BlendFlags & EDecalBlendFlags::AlphaComposite)
				{
					GetDecalBlendFunctionString(AlphaCompositeBlendFunctions[i], AttachmentString, TempOutputName, BlendFnString);
				}
				if (BlendFlags & EDecalBlendFlags::Modulate)
				{
					GetDecalBlendFunctionString(ModulateBlendFunctions[i], AttachmentString, TempOutputName, BlendFnString);
				}

				TempOut += BlendFnString;

				SourceString.insert(AssignmentPos, TempOut);
			}			
		}
	}
}

bool GenerateDeferredMobileShaders(std::string& GlslSource, GLSLCompileParameters& GLSLCompileParams, const std::string& HlslString, ReflectionData& ReflectData,
																		bool bWriteToCCHeader, bool bIsDeferred, bool bEmulatedUBs, uint32_t DecalBlendFlags)
{
	bool bHasGbufferTextures[4];
	bHasGbufferTextures[0] = HlslString.find("GENERATED_SubpassFetchAttachment0") != std::string::npos;
	bHasGbufferTextures[1] = HlslString.find("GENERATED_SubpassFetchAttachment1") != std::string::npos;
	bHasGbufferTextures[2] = HlslString.find("GENERATED_SubpassFetchAttachment2") != std::string::npos;
	bHasGbufferTextures[3] = HlslString.find("GENERATED_SubpassFetchAttachment3") != std::string::npos;

	bool bHasGBufferOutputs[4];
	bHasGBufferOutputs[0] = HlslString.find("OutProxy") != std::string::npos;
	bHasGBufferOutputs[1] = HlslString.find("OutGBufferA") != std::string::npos;
	bHasGBufferOutputs[2] = HlslString.find("OutGBufferB") != std::string::npos;
	bHasGBufferOutputs[3] = HlslString.find("OutGBufferC") != std::string::npos;

	bool bHasInputs = false;
	bool bHasOutputs = false;

	for (uint32_t i = 0; i < 4; ++i)
	{
		bHasInputs |= bHasGbufferTextures[i];
		bHasOutputs |= bHasGBufferOutputs[i];
	}

	bool bHasInOut = (bHasInputs && bHasOutputs) || HlslString.find("OutTarget0") != std::string::npos;

	if (bHasInputs || bHasOutputs)
	{
		std::string OutString;
		std::string ESVersionString = "#version 320 es";
		OutString += ESVersionString + "\n";
		OutString += "#ifdef UE_MRT_FRAMEBUFFER_FETCH\n";

		FShaderCompilerDefinitions CompileFlagsCopy = *GLSLCompileParams.TargetDesc->CompileFlags;

		// Add defines for FBF in spirv-glsl
		for (uint32_t i = 1; i < 4; ++i)
		{
			if (bHasGbufferTextures[i])
			{
				FString DefineKey = FString::Printf(TEXT("remap_ext_framebuffer_fetch%d"), i);
				FString DefineValue = FString::Printf(TEXT("%d %d"), i + 1, i);
				GLSLCompileParams.TargetDesc->CompileFlags->SetDefine(*DefineKey, *DefineValue);
			}
		}

		std::string FBFSourceString;
		if (!GenerateGlslShader(FBFSourceString, GLSLCompileParams, ReflectData, true, true, bEmulatedUBs))
		{
			return false;
		}

		size_t VersionStringPos = FBFSourceString.find(ESVersionString);
		if (VersionStringPos != std::string::npos)
		{
			FBFSourceString.replace(VersionStringPos, ESVersionString.length(), "");
		}

		*GLSLCompileParams.TargetDesc->CompileFlags = CompileFlagsCopy;

		OutString += FBFSourceString;

		// Generate PLS shader
		OutString += "#else\n";

		// Using rgb10_a2ui because r11_g11_b10 has issues with swizzle from 4 to 3 components in DXC
		static const FString GBufferOutputNames[] =
		{
			TEXT("rgb10_a2 out.var.SV_Target0"),
			TEXT("rgb10_a2 out.var.SV_Target1"),
			TEXT("rgba8 out.var.SV_Target2"),
			TEXT("rgba8 out.var.SV_Target3"),
		};

		static const FString GBufferInputNames[] =
		{
			TEXT("rgb10_a2 GENERATED_SubpassFetchAttachment0"),
			TEXT("rgb10_a2 GENERATED_SubpassFetchAttachment1"),
			TEXT("rgba8 GENERATED_SubpassFetchAttachment2"),
			TEXT("rgba8 GENERATED_SubpassFetchAttachment3"),
		};

		static const FString GBufferInOutNames[] =
		{
			TEXT("rgb10_a2 GENERATED_SubpassFetchAttachment0 out.var.SV_Target0"),
			TEXT("rgb10_a2 GENERATED_SubpassFetchAttachment1 out.var.SV_Target1"),
			TEXT("rgba8 GENERATED_SubpassFetchAttachment2 out.var.SV_Target2"),
			TEXT("rgba8 GENERATED_SubpassFetchAttachment3 out.var.SV_Target3"),
		};

		// Add defines for PLS in spirv-glsl
		{
			FString DefineKey = "";
			FString DefineValue = "";

			for (uint32_t i = 0; i < 4; ++i)
			{
				if (bHasInOut)
				{
					DefineKey = FString::Printf(TEXT("pls_io%d"), i);
					DefineValue = GBufferInOutNames[i];
				}
				else if (bHasInputs)
				{
					DefineKey = FString::Printf(TEXT("pls_in%d"), i);
					DefineValue = GBufferInputNames[i];
				}
				else
				{
					DefineKey = FString::Printf(TEXT("pls_out%d"), i);
					DefineValue = GBufferOutputNames[i];
				}

				GLSLCompileParams.TargetDesc->CompileFlags->SetDefine(*DefineKey, *DefineValue);
			}
		}

		std::string PLSSourceString;
		if (!GenerateGlslShader(PLSSourceString, GLSLCompileParams, ReflectData, false, true, bEmulatedUBs))
		{
			return false;
		}

		// Replace assignment of output to buffer 0 to additive if the output proxy additive is used
		if (HlslString.find("OutProxyAdditive") != std::string::npos)
		{
			std::string OutProxyString = "GENERATED_SubpassFetchAttachment0 =";
			std::string OutProxyModifiedString = "GENERATED_SubpassFetchAttachment0 +=";

			size_t OutProxyStringPos = PLSSourceString.find(OutProxyString);
			while (OutProxyStringPos != std::string::npos)
			{
				PLSSourceString.replace(OutProxyStringPos, OutProxyString.size(), OutProxyModifiedString);
				OutProxyStringPos = PLSSourceString.find(OutProxyString);
			}
		}

		if (DecalBlendFlags)
		{
			ModifyDecalBlending(PLSSourceString, DecalBlendFlags);
		}

		// Strip version string
		VersionStringPos = PLSSourceString.find(ESVersionString);
		if (VersionStringPos != std::string::npos)
		{
			PLSSourceString.replace(VersionStringPos, ESVersionString.length(), "", 0);
		}

		OutString += PLSSourceString;

		OutString += "#endif\n";
		GlslSource = OutString;
	}
	else
	{
		if (!GenerateGlslShader(GlslSource, GLSLCompileParams, ReflectData, true, false, bEmulatedUBs))
		{
			return false;
		}
	}

	return true;
}

static bool CompileToGlslWithShaderConductor(
	const FShaderCompilerInput&	Input,
	FShaderCompilerOutput&		Output,
	GLSLVersion					Version,
	const EShaderFrequency		Frequency,
	const FString&				PreprocessedShader,
	char*&						OutGlslShaderSource)
{
	CrossCompiler::FShaderConductorContext CompilerContext;

	const bool bDumpDebugInfo = Input.DumpDebugInfoEnabled();

	// Initialize compilation options for ShaderConductor
	CrossCompiler::FShaderConductorOptions Options(Input.Environment.CompilerFlags);
	Options.bDisableScalarBlockLayout = true;
	Options.bForceStorageImageFormat = true;
	
	// Convert input strings from FString to ANSI strings
	std::string SourceData(TCHAR_TO_UTF8(*PreprocessedShader));
	std::string FileName(TCHAR_TO_UTF8(*Input.VirtualSourceFilePath));
	std::string EntryPointName(TCHAR_TO_UTF8(*Input.EntryPointName));
	
	bool bEmulatedUBs = Input.Environment.CompilerFlags.Contains(CFLAG_UseEmulatedUB);

	// HLSL framebuffer declarations. Used to modify HLSL input source.
	const ANSICHAR* HlslFrameBufferDeclarations =
		"float4 gl_FragColor;\n"
		"float4 gl_LastFragColorARM;\n"
		"float gl_LastFragDepthARM;\n"
		"bool ARM_shader_framebuffer_fetch;\n"
		"bool ARM_shader_framebuffer_fetch_depth_stencil;\n"
		"float4 FramebufferFetchES2()\n"
		"{\n"
		"  if (!ARM_shader_framebuffer_fetch)\n"
		"  {\n"
		"    return gl_FragColor;\n"
		"  }\n"
		"  else\n"
		"  {\n"
		"    return gl_LastFragColorARM;\n"
		"  }\n"
		"}\n"
		"float DepthbufferFetchES2()\n"
		"{\n"
		"  return (ARM_shader_framebuffer_fetch_depth_stencil == 0 ? 0.0 : gl_LastFragDepthARM);\n"
		"}\n"
		;

	if (SourceData.find("DepthbufferFetchES2") != std::string::npos ||
		SourceData.find("FramebufferFetchES2") != std::string::npos)
	{
		SourceData = HlslFrameBufferDeclarations + SourceData;
	}
	
	// Inject additional macro definitions to circumvent missing features: external textures
	FShaderCompilerDefinitions AdditionalDefines;
	AdditionalDefines.SetDefine(TEXT("TextureExternal"), TEXT("Texture2D"));

	uint32_t BlendFlags = GetDecalBlendFlags(Input);

	// Load shader source into compiler context
	CompilerContext.LoadSource(SourceData.c_str(), FileName.c_str(), EntryPointName.c_str(), Frequency, &AdditionalDefines);

	bool bCompilationFailed = false;

	// Compile HLSL source to SPIR-V binary
	TArray<uint32> SpirvData;
	
	if (!bCompilationFailed && !CompilerContext.CompileHlslToSpirv(Options, SpirvData))
	{
		// Flush compile errors
		CompilerContext.FlushErrors(Output.Errors);
		bCompilationFailed = true;
	}

	// Reduce arrays with const accessors to structs, which will then be packed to an array by GL cross compile
	if(!bCompilationFailed && !Options.bDisableOptimizations && (Version == GLSL_ES3_1_ANDROID || Version == GLSL_150_ES3_1))
	{
		const char* OptArgs[] = { "--reduce-const-array-to-struct", "--convert-composite-to-op-access-chain-pass"};
		if (!CompilerContext.OptimizeSpirv(SpirvData, OptArgs, UE_ARRAY_COUNT(OptArgs)))
		{
			UE_LOGF(LogOpenGLShaderCompiler, Error, "Failed to apply reduce-const-array-to-struct for Android");
			return false;
		} 
	}

	// For Android run an additional pass to patch spirv to be compatible across drivers
	if (!bCompilationFailed && (Version == GLSL_ES3_1_ANDROID || Version == GLSL_150_ES3_1))
	{
		const char* OptArgs[] = {"--adv-interface-variable-scalar-replacement=skip-matrices"};
		if (!CompilerContext.OptimizeSpirv(SpirvData, OptArgs, UE_ARRAY_COUNT(OptArgs)))
		{
			UE_LOGF(LogOpenGLShaderCompiler, Error, "Failed to apply interface-variable-scalar-replacement for Android");
			return false;
		}
	}

	if (!bCompilationFailed)
	{
		ReflectionData ReflectData;
		CrossCompiler::FHlslccHeaderWriter CCHeaderWriter;
		
		// Now perform reflection on the SPIRV and tweak any decorations that we need to.
		// This used to be done via JSON, but that was slow and alloc happy so use SPIRV-Reflect instead.
		spv_reflect::ShaderModule Reflection(SpirvData.Num() * sizeof(uint32), SpirvData.GetData());
		check(Reflection.GetResult() == SPV_REFLECT_RESULT_SUCCESS);

		const ANSICHAR* SPIRV_DummySamplerName = CrossCompiler::FShaderConductorContext::GetIdentifierTable().DummySampler;

		ParseReflectionData(Input, CCHeaderWriter, ReflectData, SpirvData, Reflection, SPIRV_DummySamplerName, Frequency, bEmulatedUBs);
		
		// Step 2 : End of reflection
		// 
		// Overwrite updated SPIRV code
		SpirvData = TArray<uint32>(Reflection.GetCode(), Reflection.GetCodeSize() / 4);

		if (bDumpDebugInfo)
		{
			// SPIR-V file (Binary)
			DumpDebugShaderBinary(Input, SpirvData.GetData(), SpirvData.Num() * sizeof(uint32), TEXT("spv"));
		}

		CrossCompiler::FShaderConductorTarget TargetDesc;

		TargetDesc.CompileFlags->SetDefine(TEXT("relax_nan_checks"), 1);

		switch (Version)
		{
		case GLSL_150_ES3_1:
		case GLSL_ES3_1_ANDROID:
		default:
			TargetDesc.CompileFlags->SetDefine(TEXT("force_flattened_io_blocks"), 1);
			TargetDesc.CompileFlags->SetDefine(TEXT("emit_uniform_buffer_as_plain_uniforms"), 1);
			TargetDesc.CompileFlags->SetDefine(TEXT("pad_ubo_blocks"), 1);
			// TODO: Currently disabled due to bug when assigning an array to temporary variable
			//TargetDesc.CompileFlags.SetDefine(TEXT("force_temporary"), 1);

			// If we have mobile multiview define set then set the view count and enable extension
			const bool bMultiView = Input.Environment.GetCompileArgument(TEXT("MOBILE_MULTI_VIEW"), false);
			if (Frequency == SF_Vertex && bMultiView)
			{
				TargetDesc.CompileFlags->SetDefine(TEXT("ovr_multiview_view_count"), 2);
			}

			if (Version == GLSL_150_ES3_1)
			{
				TargetDesc.CompileFlags->SetDefine(TEXT("force_glsl_clipspace"), 1);
			}
			TargetDesc.Language = CrossCompiler::EShaderConductorLanguage::Essl;
			TargetDesc.Version = 320;
			break;
		}

		TSet<FString> ExternalTextures;
		int32 Pos = 0;
		static constexpr FStringView TextExternal = TEXTVIEW("TextureExternal");
		
		do
		{
			Pos = PreprocessedShader.Find(TextExternal, ESearchCase::CaseSensitive, ESearchDir::FromStart, Pos + TextExternal.Len());
			if (Pos != INDEX_NONE)
			{
				FStringView TextureExternalName;

				TextureExternalName = FindNextHLSLDefinitionOfType(FStringView(&PreprocessedShader[Pos]), FStringView(&PreprocessedShader[Pos+TextExternal.Len()]));
				if (!TextureExternalName.IsEmpty())
				{
					ExternalTextures.Add(FString(TextureExternalName));
				}
			}
		}
		while (Pos != INDEX_NONE);

		// Define type renaming callback after all external texture types have been gathered
		TargetDesc.VariableTypeRenameCallback = [&ExternalTextures](const FAnsiStringView& VariableName, const FAnsiStringView& TypeName, FString& OutRenamedTypeName) -> bool
		{
			auto WideVarName = StringCast<TCHAR>(VariableName.GetData());
			for (const FString& ExternalTex : ExternalTextures)
			{
				if (FCStringWide::Strstr(WideVarName.Get(), *ExternalTex) &&
					FCStringWide::Strstr(WideVarName.Get(), TEXT("_SamplerP")))
				{
					OutRenamedTypeName = TEXT("samplerExternalOES");
					return true;
				}
			}
			return false;
		};

		GLSLCompileParameters GLSLCompileParams;

		GLSLCompileParams.CompilerContext = &CompilerContext;
		GLSLCompileParams.CCHeaderWriter = &CCHeaderWriter;
		GLSLCompileParams.TargetDesc = &TargetDesc;
		GLSLCompileParams.Options = &Options;

		GLSLCompileParams.Output = &Output;
		GLSLCompileParams.SpirvData = MoveTemp(SpirvData);

		GLSLCompileParams.Frequency = Frequency;
		GLSLCompileParams.SPIRV_DummySamplerName = SPIRV_DummySamplerName;

		std::string GlslSource;

		// Handle PLS and FBF in OpenGL
		if (Version == GLSL_ES3_1_ANDROID &&
			Input.Environment.GetCompileArgument(TEXT("USE_GLES_FBF_DEFERRED"), false))
		{
			bCompilationFailed = !GenerateDeferredMobileShaders(GlslSource, GLSLCompileParams, SourceData, ReflectData, true, false, bEmulatedUBs, BlendFlags);
		}
		else
		{
			bCompilationFailed = !GenerateGlslShader(GlslSource, GLSLCompileParams, ReflectData, true, false, bEmulatedUBs);
		}

		// Generate meta data for CCHeader
		CCHeaderWriter.WriteSourceInfo(*Input.VirtualSourceFilePath, *Input.EntryPointName);
		CCHeaderWriter.WriteCompilerInfo();

		if (GlslSource.length() > 0)
		{
			const FString MetaData = CCHeaderWriter.ToString();

			// Merge meta data and GLSL source to output string
			const int32 GlslShaderSourceLen = MetaData.Len() + static_cast<int32>(GlslSource.size()) + 1;
			OutGlslShaderSource = (char*)malloc(GlslShaderSourceLen);
			FCStringAnsi::Snprintf(OutGlslShaderSource, GlslShaderSourceLen, "%s%s", TCHAR_TO_ANSI(*MetaData), GlslSource.c_str());
		}
	}

	if (bDumpDebugInfo && OutGlslShaderSource != nullptr)
	{
		const TCHAR* ShaderFileExt = CrossCompiler::FShaderConductorContext::GetShaderFileExt(CrossCompiler::EShaderConductorLanguage::Essl, Frequency);
		DumpDebugShaderText(Input, OutGlslShaderSource, FCStringAnsi::Strlen(OutGlslShaderSource), ShaderFileExt);
	}

	return !bCompilationFailed;
}

inline bool OpenGLSupportsShaderFrequency(EShaderFrequency InFrequency)
{
	return InFrequency == SF_Vertex
		|| InFrequency == SF_Pixel
		|| InFrequency == SF_Geometry
		|| InFrequency == SF_Compute;
}

/**
 * Compile a shader for OpenGL on Windows.
 * @param Input - The input shader code and environment.
 * @param PreprocessOutput - The output of the shader preprocessing phase (see above)
 * @param Output - Contains shader compilation results upon return.
 * @param Version - Target GLSL version for this compilation
 */
void CompileOpenGLShader(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& InPreprocessOutput, FShaderCompilerOutput& Output, GLSLVersion Version)
{
	FString EntryPointName = Input.EntryPointName;
	FString PreprocessedSource(InPreprocessOutput.GetSourceViewWide());

	FShaderParameterParser::FPlatformConfiguration PlatformConfiguration;
	FShaderParameterParser ShaderParameterParser(PlatformConfiguration);
	if (!ShaderParameterParser.ParseAndModify(Input, Output.Errors, PreprocessedSource))
	{
		// The FShaderParameterParser will add any relevant errors.
		return;
	}

	if (ShaderParameterParser.DidModifyShader())
	{
		Output.ModifiedShaderSource = PreprocessedSource;
	}

	const bool bDumpDebugInfo = Input.DumpDebugInfoEnabled();
	
	char* GlslShaderSource = nullptr;
	char* ErrorLog = nullptr;

	const EShaderFrequency Frequency = static_cast<EShaderFrequency>(Input.Target.Frequency);
	if (!OpenGLSupportsShaderFrequency(Frequency))
	{
		Output.bSucceeded = false;
		FShaderCompilerError& NewError = Output.Errors.AddDefaulted_GetRef();
		NewError.StrippedErrorMessage = FString::Printf(
			TEXT("%s shaders not supported for use in OpenGL."),
			CrossCompiler::GetFrequencyName(Frequency)
			);
		return;
	}

	const bool bCompilationSucceeded = CompileToGlslWithShaderConductor(Input, Output, Version, Frequency, PreprocessedSource, GlslShaderSource);

	if (bCompilationSucceeded)
	{
		// print resulting glsl to debug output if compiling from a debug dumped usf file
		if (EnumHasAnyFlags(Input.DebugInfoFlags, EShaderDebugInfoFlags::CompileFromDebugUSF))
		{
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%s\n"), ANSI_TO_TCHAR(GlslShaderSource));
		}

#if VALIDATE_GLSL_WITH_DRIVER
		PrecompileShaderInternal(Output, Input, ShaderParameterParser, GlslShaderSource, Version);
#else // VALIDATE_GLSL_WITH_DRIVER
		int32 SourceLen = FCStringAnsi::Strlen(GlslShaderSource);
		Output.Target = Input.Target;
		BuildShaderOutputInternal(Output, Input, ShaderParameterParser, GlslShaderSource, SourceLen, Version);
#endif // VALIDATE_GLSL_WITH_DRIVER

		if (bDumpDebugInfo && GlslShaderSource != nullptr)
		{
			const TCHAR* ShaderFileExt = CrossCompiler::FShaderConductorContext::GetShaderFileExt(CrossCompiler::EShaderConductorLanguage::Essl, Frequency);
			const FString DumpedGlslFile = FString::Printf(TEXT("%s/Output.%s"), *Input.DumpDebugInfoPath, ShaderFileExt);
			if (const TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*DumpedGlslFile)))
			{
				FileWriter->Serialize(GlslShaderSource, FCStringAnsi::Strlen(GlslShaderSource));
				FileWriter->Close();
			}
		}
	}

	if (GlslShaderSource)
	{
		free(GlslShaderSource);
	}
	if (ErrorLog)
	{
		free(ErrorLog);
	}

	// Do not validate as global halfN != UB's halfN
	//ShaderParameterParser.ValidateShaderParameterTypes(Input, Output);
}

static void FillDeviceCapsOfflineCompilationInternal(struct FDeviceCapabilities& Capabilities, const GLSLVersion ShaderVersion)
{
	FMemory::Memzero(Capabilities);

	if (ShaderVersion == GLSL_ES3_1_ANDROID)
	{
		Capabilities.TargetPlatform = EPlatformType::Android;
	}
	else
	{
		Capabilities.TargetPlatform = EPlatformType::Desktop;
	}
}

static bool MoveHashLines(FAnsiString& Destination, FAnsiString& Source)
{
	int32 Index = 0;
	int32 LineStart = 0;

	bool bFound = false;
	while (Index != INDEX_NONE && !bFound)
	{
		LineStart = Index;
		Index = Source.Find("\n", ESearchCase::IgnoreCase, ESearchDir::FromStart, Index);

		for (int32 i = LineStart; i < Index; ++i)
		{
			const auto CharValue = Source[i];
			if (CharValue == '#')
			{
				break;
			}
			else if (!FChar::IsWhitespace(CharValue))
			{
				bFound = true;
				break;
			}
		}

		++Index;
	}

	if (bFound)
	{
		Destination.Append(Source.Left(LineStart));
		Source.RemoveAt(0, LineStart);
	}

	return bFound;
}

static FAnsiString PrepareCodeForOfflineCompilationInternal(const GLSLVersion ShaderVersion, EShaderFrequency Frequency, const ANSICHAR* InShaderSource)
{
	FAnsiString OriginalShaderSource(InShaderSource);
	FAnsiString StrOutSource;

	FDeviceCapabilities Capabilities;
	FillDeviceCapsOfflineCompilationInternal(Capabilities, ShaderVersion);

	// Whether we need to emit mobile multi-view code or not.
	const bool bEmitMobileMultiView = OriginalShaderSource.Find("gl_ViewID_OVR") != INDEX_NONE;

	// Whether we need to emit texture external code or not.
	const bool bEmitTextureExternal = OriginalShaderSource.Find("samplerExternalOES") != INDEX_NONE;

	if (Capabilities.TargetPlatform == EPlatformType::Android || Capabilities.TargetPlatform == EPlatformType::Web)
	{
		const ANSICHAR* ES320Version = "#version 320 es";
		StrOutSource.Append(ES320Version);
		StrOutSource.Append("\n");
		OriginalShaderSource.RemoveFromStart(ES320Version);
	}

	// The incoming glsl may have preprocessor code that is dependent on defines introduced via the engine.
	// This is the place to insert such engine preprocessor defines, immediately after the glsl version declarati
	if (bEmitMobileMultiView)
	{
		MoveHashLines(StrOutSource, OriginalShaderSource);

		StrOutSource.Append("\n\n");
		StrOutSource.Append("#extension GL_OVR_multiview2 : enable\n");
		StrOutSource.Append("\n\n");
	}

	if (bEmitTextureExternal)
	{
		MoveHashLines(StrOutSource, OriginalShaderSource);
		StrOutSource.Append("#define samplerExternalOES sampler2D\n");
	}

	// Move version tag & extensions before beginning all other operations
	MoveHashLines(StrOutSource, OriginalShaderSource);

	// OpenGL SM5 shader platforms require location declarations for the layout, but don't necessarily use SSOs
	if (Capabilities.TargetPlatform == EPlatformType::Desktop)
	{
		StrOutSource.Append("#define INTERFACE_BLOCK(Pos, Interp, Modifiers, Semantic, PreType, PostType) layout(location=Pos) Interp Modifiers struct { PreType PostType; }\n");
	}
	else
	{
		StrOutSource.Append("#define INTERFACE_BLOCK(Pos, Interp, Modifiers, Semantic, PreType, PostType) layout(location=Pos) Modifiers Semantic { PreType PostType; }\n");
	}

	if (Capabilities.TargetPlatform == EPlatformType::Desktop)
	{
		// If we're running <= featurelevel es3.1 shaders then enable this extension which adds support for uintBitsToFloat etc.
		if (StrOutSource.Contains("#version 150"))
		{
			StrOutSource.Append("\n\n");
			StrOutSource.Append("#extension GL_ARB_gpu_shader5 : enable\n");
			StrOutSource.Append("\n\n");
		}
	}

	StrOutSource.Append("#define HLSLCC_DX11ClipSpace 1 \n");

	// Append the possibly edited shader to the one we will compile.
	// This is to make it easier to debug as we can see the whole
	// shader source.
	StrOutSource.Append("\n\n");
	StrOutSource.Append(OriginalShaderSource);

	return StrOutSource;
}

static bool PlatformSupportsOfflineCompilationInternal(const GLSLVersion ShaderVersion)
{
	return ShaderVersion == GLSL_ES3_1_ANDROID;
}

static void CompileOfflineInternal(const FShaderCompilerInput& Input, FShaderCompilerOutput& Output, const GLSLVersion ShaderVersion, const ANSICHAR* InShaderSource)
{
	const bool bSupportsOfflineCompilation = PlatformSupportsOfflineCompilationInternal(ShaderVersion);

	if (!bSupportsOfflineCompilation)
	{
		return;
	}

	FAnsiString ShaderSource = PrepareCodeForOfflineCompilationInternal(ShaderVersion, (EShaderFrequency)Input.Target.Frequency, InShaderSource);

	PlatformCompileOfflineInternal(Input, Output, *ShaderSource, ShaderVersion);
}

static void PlatformCompileOfflineInternal(const FShaderCompilerInput& Input, FShaderCompilerOutput& ShaderOutput, const ANSICHAR* ShaderSource, const GLSLVersion ShaderVersion)
{
	if (ShaderVersion == GLSL_ES3_1_ANDROID)
	{
		CompileShaderOffline(Input, ShaderOutput, ShaderSource, FPlatformString::Strlen(ShaderSource), false);
	}
}
