// Copyright Epic Games, Inc. All Rights Reserved.
//

#include "ShaderFormatOpenGL.h"

#include "DXCWrapper.h"
#include "ShaderCompilerCommon.h"
#include "ShaderPreprocessor.h"
#include "ShaderPreprocessTypes.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/IShaderFormatModule.h"
#include "ShaderCore.h"
#include "ShaderCompilerCore.h"

static FName NAME_GLSL_150_ES3_1(TEXT("GLSL_150_ES31"));
static FName NAME_GLSL_ES3_1_ANDROID(TEXT("GLSL_ES3_1_ANDROID"));

extern bool ShouldUseDXC(FShaderCompilerFlags Flags);

extern void CompileOpenGLShader(
	const FShaderCompilerInput& Input,
	const FShaderPreprocessOutput& InPreprocessOutput,
	FShaderCompilerOutput& Output,
	GLSLVersion Version);

/** Version for shader format, this becomes part of the DDC key. */
static const FGuid UE_SHADER_GLSL_VER = FGuid("16563EF4-3474-4296-8DA3-AEFF2B0547F1");

class FShaderFormatGLSL : public UE::ShaderCompilerCommon::FBaseShaderFormat 
{
	static void CheckFormat(FName Format)
	{
		check(Format == NAME_GLSL_150_ES3_1 || Format == NAME_GLSL_ES3_1_ANDROID);
	}

	const uint32 ShaderConductorVersionHash;

public:
	FShaderFormatGLSL(uint32 InShaderConductorVersionHash)
		: ShaderConductorVersionHash(InShaderConductorVersionHash)
	{
	}

	virtual uint32 GetVersion(FName Format) const override
	{
		CheckFormat(Format);
		FGuid GLSLVersion{};
		if (ensure(Format == NAME_GLSL_150_ES3_1 || Format == NAME_GLSL_ES3_1_ANDROID))
		{
			GLSLVersion = UE_SHADER_GLSL_VER;
		}

		uint32 Version = GetTypeHash(GLSLVersion);
		Version = HashCombine(Version, GetTypeHash(ShaderConductorVersionHash));

		return Version;
	}
	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Add(NAME_GLSL_150_ES3_1);
		OutFormats.Add(NAME_GLSL_ES3_1_ANDROID);
	}

	static GLSLVersion TranslateFormatNameToEnum(FName Format)
	{
		if (Format == NAME_GLSL_150_ES3_1)
		{
			return GLSL_150_ES3_1;
		}
		else if (Format == NAME_GLSL_ES3_1_ANDROID)
		{
			return GLSL_ES3_1_ANDROID;
		}
		else
		{
			check(0);
			return GLSL_MAX;
		}
	}

	virtual void ModifyShaderCompilerInput(FShaderCompilerInput& Input) const override
	{
		GLSLVersion Version = TranslateFormatNameToEnum(Input.ShaderFormat);
		switch (Version)
		{
		case GLSL_ES3_1_ANDROID:
			Input.Environment.SetDefine(TEXT("COMPILER_GLSL_ES3_1"), 1);
			Input.Environment.SetDefine(TEXT("ES3_1_PROFILE"), 1);
			Input.Environment.SetDefine(TEXT("PLATFORM_SUPPORTS_RELAXED_PRECISION"), 1);
			break;

		case GLSL_150_ES3_1:
			Input.Environment.SetDefine(TEXT("COMPILER_GLSL"), 1);
			Input.Environment.SetDefine(TEXT("ES3_1_PROFILE"), 1);
			Input.Environment.SetDefine(TEXT("row_major"), TEXT(""));
			break;

		default:
			check(0);
		}
		Input.Environment.SetDefine(TEXT("OPENGL_PROFILE"), 1);

		constexpr int32 HlslccSubstituteValueForDxc = 2;
		Input.Environment.SetDefine(TEXT("COMPILER_HLSLCC"), HlslccSubstituteValueForDxc);
		Input.Environment.SetDefine(TEXT("COMPILER_SUPPORTS_ATTRIBUTES"), (uint32)1);

		if (Input.Environment.FullPrecisionInPS || (IsValidRef(Input.SharedEnvironment) && Input.SharedEnvironment->FullPrecisionInPS))
		{
			Input.Environment.SetDefine(TEXT("FORCE_FLOATS"), (uint32)1);
		}
	}

	virtual void CompilePreprocessedShader(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& PreprocessOutput, FShaderCompilerOutput& Output) const override
	{
		CheckFormat(Input.ShaderFormat);
		CompileOpenGLShader(Input, PreprocessOutput, Output, TranslateFormatNameToEnum(Input.ShaderFormat));
	}

	virtual const TCHAR* GetPlatformIncludeDirectory() const override
	{
		return TEXT("GL");
	}
};

/**
 * Module for OpenGL shaders
 */

static IShaderFormat* Singleton = nullptr;

class FShaderFormatOpenGLModule : public IShaderFormatModule, public FShaderConductorModuleWrapper
{
public:
	virtual ~FShaderFormatOpenGLModule() override
	{
		delete Singleton;
		Singleton = nullptr;
	}
	virtual IShaderFormat* GetShaderFormat() override
	{
		if (!Singleton)
		{
			Singleton = new FShaderFormatGLSL(FShaderConductorModuleWrapper::GetModuleVersionHash());
		}
		return Singleton;
	}
};

IMPLEMENT_MODULE( FShaderFormatOpenGLModule, ShaderFormatOpenGL);
