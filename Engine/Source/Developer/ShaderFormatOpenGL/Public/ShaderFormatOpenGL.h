// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "RHIDefinitions.h"
#include "Templates/SharedPointer.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#endif

class FArchive;
class FShaderCompilerFlags;
struct FShaderCompilerInput;
struct FShaderCompilerOutput;

#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

enum GLSLVersion 
{
	GLSL_150_REMOVED,
	GLSL_430_REMOVED,
	GLSL_ES2_REMOVED,
	GLSL_ES2_WEBGL_REMOVED,
	GLSL_150_ES2_DEPRECATED,	// ES2 Emulation
	GLSL_150_ES2_NOUB_DEPRECATED,	// ES2 Emulation with NoUBs
	GLSL_150_ES3_1,	// ES3.1 Emulation
	GLSL_ES2_IOS_REMOVED,
	GLSL_310_ES_EXT_REMOVED,
	GLSL_ES3_1_ANDROID,

	GLSL_MAX
};
