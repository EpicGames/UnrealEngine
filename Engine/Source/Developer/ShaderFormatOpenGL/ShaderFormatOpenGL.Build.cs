// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ShaderFormatOpenGL : ModuleRules
{
	public ShaderFormatOpenGL(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"TargetPlatform",
				"OpenGLDrv"
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"RenderCore",
				"ShaderCompilerCommon",
				"ShaderPreprocessor",
				"RHI" // @todo platplug: this is caused by the DataDrivenShaderPlatformInfo stuff - maybe it should move to somewhere else, like RenderCore?
			}
			);

		AddEngineThirdPartyPrivateStaticDependencies(Target, 
			"OpenGL"
			);

        if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
        {
            AddEngineThirdPartyPrivateStaticDependencies(Target, "SDL3");
        }
		if (Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Linux)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "SPIRVReflect");
		}
	}
}
